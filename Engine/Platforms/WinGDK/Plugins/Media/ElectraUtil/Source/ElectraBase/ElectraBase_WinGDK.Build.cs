// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ElectraBase_WinGDK : ElectraBase
	{
		public ElectraBase_WinGDK(ReadOnlyTargetRules Target) : base(Target)
		{
		}
	}
}
