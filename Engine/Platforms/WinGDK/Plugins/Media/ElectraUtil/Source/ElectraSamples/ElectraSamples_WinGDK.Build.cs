// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ElectraSamples_WinGDK : ElectraSamples
	{
		public ElectraSamples_WinGDK(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.Add("D3D11RHI");
			PrivateDefinitions.Add("ELECTRA_HAVE_DX11");
		}
	}
}
