{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Electra Player Utilities",
	"Description" : "Reusable Base Components for Electra Player Media Playback",
	"Category" : "Media Players",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "https://docs.unrealengine.com/en-US/Engine/MediaFramework/Overview",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules": [
		{
			"Name": "ElectraBase",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64", "Mac", "IOS", "TVOS", "Android", "Linux" ],
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "ElectraSamples",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64", "Mac", "IOS", "TVOS", "Android", "Linux" ],
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "ElectraHTTPStream",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64", "Mac", "IOS", "TVOS", "Android", "Linux" ],
			"TargetDenyList": [ "Server" ]
		}
	]
}
