// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ElectraBase : ModuleRules
	{
		public ElectraBase(ReadOnlyTargetRules Target) : base(Target)
		{
			IWYUSupport = IWYUSupport.KeepAsIsForNow;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
                    "Engine"
				});
		}
	}
}
