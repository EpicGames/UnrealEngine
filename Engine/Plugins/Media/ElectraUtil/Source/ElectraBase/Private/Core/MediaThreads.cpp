// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/MediaThreads.h"
#include "Core/MediaMessageQueue.h"

#include "HAL/RunnableThread.h"
#include "Misc/ScopeLock.h"


namespace MediaRunnablePrivate
{

static TMediaMessageQueueDynamicNoTimeout<TFunction<void()>> AsyncFunctionsToRun;
static volatile bool bEndForShutdown = false;
static FMediaThread* AsyncExecutionThread = nullptr;
static void AsyncWorkerThreadFN()
{
	while(!bEndForShutdown)
	{
		TFunction<void()> AsyncRunFN = AsyncFunctionsToRun.ReceiveMessage();
		if (AsyncRunFN)
		{
			AsyncRunFN();
		}
	}
}

}

void FMediaRunnable::Startup()
{
	MediaRunnablePrivate::bEndForShutdown = false;
	MediaRunnablePrivate::AsyncExecutionThread = new FMediaThread("Electra::ExecAsync");
	MediaRunnablePrivate::AsyncExecutionThread->ThreadStart(FMediaRunnable::FStartDelegate::CreateStatic(&MediaRunnablePrivate::AsyncWorkerThreadFN));
}

void FMediaRunnable::Shutdown()
{
	check(MediaRunnablePrivate::AsyncExecutionThread);

	TFunction<void()> FinishTask = []()
	{
		MediaRunnablePrivate::bEndForShutdown = true;
	};
	EnqueueAsyncTask(MoveTemp(FinishTask));

	MediaRunnablePrivate::AsyncExecutionThread->ThreadWaitDone();
	delete MediaRunnablePrivate::AsyncExecutionThread;
	MediaRunnablePrivate::AsyncExecutionThread = nullptr;
}

void FMediaRunnable::EnqueueAsyncTask(TFunction<void()>&& InFunctionToExecuteOnAsyncThread)
{
	MediaRunnablePrivate::AsyncFunctionsToRun.SendMessage(MoveTemp(InFunctionToExecuteOnAsyncThread));
}


// ----------------------------------------------------------------------------
/**
 * Creates a new thread.
 * Actually only the parameters are set, but no actual thread is created.
 * We leave this to the Start() function so it becomes possible to change parameters after construction.
 *
 * @param InCoreAffinityMask
 * @param InPriority
 * @param InStackSize
 * @param InThreadName
 *
 * @return New runnable object
 */
FMediaRunnable* FMediaRunnable::Create(int32 InCoreAffinityMask, EThreadPriority InPriority, uint32 InStackSize, const FString& InThreadName)
{
	uint64 AffinityMask = (uint32)InCoreAffinityMask;
	if (InCoreAffinityMask == 0 || InCoreAffinityMask == -1)
	{
		AffinityMask = FPlatformAffinity::GetNoAffinityMask();
	}

	FMediaRunnable* t = new FMediaRunnable;
	if (t)
	{
		t->ThreadPriority = InPriority;
		t->InitialCoreAffinity = AffinityMask;
		t->StackSize = Align((uint32)InStackSize, 16384);
		t->ThreadName = InThreadName;
		t->bIsStarted = false;
	}
	return t;
}

//-----------------------------------------------------------------------------
/**
 *
 */
void FMediaRunnable::Destroy(FMediaRunnable* InMediaRunnable)
{
	delete InMediaRunnable;
}

// ----------------------------------------------------------------------------
/**
 * Creates and starts the thread with the current settings.
 *
 * @param Entry
 * @param bWaitRunning
 */
void FMediaRunnable::Start(FMediaRunnable::FStartDelegate Entry, bool bWaitRunning)
{
	// Remember entry function
	EntryFunction = Entry;

	StartInternal();

	// Optionally wait for thread startup
	if (bWaitRunning)
	{
		SignalRunning->Wait();
	}
}



// ----------------------------------------------------------------------------
/**
 * Sets a 'done' signal to signal when the thread is being destroyed.
 *
 * @param pDoneSignal
 */
void FMediaRunnable::SetDoneSignal(FMediaEvent* InDoneSignal)
{
	FScopeLock Lock(&StateAccessMutex);
	DoneSignal = InDoneSignal;
}


// ----------------------------------------------------------------------------
/**
 * Creates and starts the thread with the current settings.
 */
void FMediaRunnable::StartInternal()
{
	// We ignore stack size, priority and affinity here on purpose. Prio and affinity are all set to defaults and stack sizes are
	// often set too small so we rather go with defaults.
	MediaThreadRunnable = FRunnableThread::Create(this, *ThreadName);
	checkf(MediaThreadRunnable, TEXT("Could not create RunnableThread!"));
	if (MediaThreadRunnable)
	{
		bIsStarted = true;
	}
}



// ----------------------------------------------------------------------------
/**
 * Thread startup function.
 */
uint32 FMediaRunnable::Run()
{
	SignalRunning->Trigger();
	EntryFunction.Execute();
	return 0;
}


// ----------------------------------------------------------------------------
/**
 * Thread completion function.
 * Only notifies optionally set signal which FMediaThread::ThreadWaitDone() is waiting for.
 * Actual thread deletion will happen from there.
 */
void FMediaRunnable::Exit()
{
	StateAccessMutex.Lock();
	bIsStarted = false;
	FMediaEvent* Sig = DoneSignal;
	DoneSignal = nullptr;
	StateAccessMutex.Unlock();
	if (Sig)
	{
		Sig->Signal();
	}
}


// ----------------------------------------------------------------------------
/**
 * Thread constructor
 */
FMediaRunnable::FMediaRunnable()
{
	MediaThreadRunnable = nullptr;

	ThreadPriority = TPri_Normal;
	StackSize = 0;

	DoneSignal = nullptr;
	bIsStarted = false;

	InitialCoreAffinity = FPlatformAffinity::GetNoAffinityMask();
	SignalRunning = FPlatformProcess::GetSynchEventFromPool();
}



// ----------------------------------------------------------------------------
/**
 * Thread destructor.
 */
FMediaRunnable::~FMediaRunnable()
{
	delete MediaThreadRunnable;
	FPlatformProcess::ReturnSynchEventToPool(SignalRunning);
}





// ----------------------------------------------------------------------------
/**
 * Changes the priority of the thread.
 *
 * @param newPriority
 *               New thread priority
 *
 * @return Previous priority
 */
EThreadPriority FMediaRunnable::ChangePriority(EThreadPriority newPriority)
{
	checkf(!bIsStarted, TEXT("FMediaRunnable::ChangePriority: Missing interface how to change thread priortity when thread was already started"));
	EThreadPriority prevPrio = ThreadPriority;
	ThreadPriority = newPriority;
	return prevPrio;
}


// ----------------------------------------------------------------------------
/**
 * Give a new name to the thread.
 *
 * @param pNewName New name of the thread.
 */
void FMediaRunnable::SetName(const FString& InThreadName)
{
	checkf(!bIsStarted, TEXT("FMediaRunnable::Name: Cannot set name when thread is already running"));
	ThreadName = InThreadName;
}

// ----------------------------------------------------------------------------
/**
 * Constructor
 *
 * @param pName
 * @param osPriority
 * @param stackSize
 */
FMediaThread::FMediaThread(const char* AnsiName)
	: MediaRunnable(nullptr)
	, Priority(TPri_Normal)
	, StackSize(0)
	, CoreAffinity(0)
	, bIsStarted(false)
	, bWaitDoneOnDelete(false)
{
	ThreadSetName(AnsiName);
}



// ----------------------------------------------------------------------------
/**
 * Destructor
 */
FMediaThread::~FMediaThread()
{
	if (bWaitDoneOnDelete && bIsStarted)
	{
		SigDone.Wait();
	}
	if (MediaRunnable)
	{
		FMediaRunnable::Destroy(MediaRunnable);
	}
}


// ----------------------------------------------------------------------------
/**
 * Set a thread priority other than the one given to the constructor.
 *
 * @param osPriority
 */
void FMediaThread::ThreadSetPriority(EThreadPriority InPriority)
{
	Priority = InPriority;
	if (MediaRunnable)
	{
		MediaRunnable->ChangePriority(Priority);
	}
}



// ----------------------------------------------------------------------------
/**
 * Sets a thread core affinity mask before starting the thread.
 *
 * @param coreAffinity
 */
void FMediaThread::ThreadSetCoreAffinity(int32 coreAffinity)
{
	checkf(!bIsStarted, TEXT("not while the thread is already running!"));
	CoreAffinity = coreAffinity;
}



// ----------------------------------------------------------------------------
/**
 * Set a thread stack size other than the one given to the constructor before starting the thread.
 *
 * @param stackSize
 */
void FMediaThread::ThreadSetStackSize(uint32 stackSize)
{
	checkf(!bIsStarted, TEXT("not while the thread is already running!"));
	StackSize = stackSize;
}



// ----------------------------------------------------------------------------
/**
 * Set a thread name other than the one given to the constructor before starting the thread.
 *
 * @param pName
 */
void FMediaThread::ThreadSetName(const char* InAnsiThreadName)
{
	checkf(!bIsStarted, TEXT("not while the thread is already running!"));
	ThreadName = FString(InAnsiThreadName);
}



// ----------------------------------------------------------------------------
/**
 * Sets a new thread name once the thread is running.
 *
 * @param pName
 */
void FMediaThread::ThreadRename(const char* InAnsiThreadName)
{
	checkf(bIsStarted, TEXT("only while the thread is already running!"));
	ThreadName = FString(InAnsiThreadName);
	MediaRunnable->SetName(ThreadName);
}



// ----------------------------------------------------------------------------
/**
 * Sets whether or not the destructor needs to wait for the thread to have finished. Defaults to false. Useful when using this as a member variable.
 *
 * @param bWait
 */
void FMediaThread::ThreadWaitDoneOnDelete(bool bWait)
{
	bWaitDoneOnDelete = bWait;
}



// ----------------------------------------------------------------------------
/**
 * Waits for the thread to have finished.
 */
void FMediaThread::ThreadWaitDone(void)
{
	checkf(bIsStarted, TEXT("thread not started"));
	SigDone.Wait();
}



// ----------------------------------------------------------------------------
/**
 * Thread start function.
 *
 * @param entryFunction
 */
void FMediaThread::ThreadStart(FMediaRunnable::FStartDelegate EntryFunction)
{
	MediaRunnable = FMediaRunnable::Create(CoreAffinity, Priority, StackSize, ThreadName);
	MediaRunnable->SetDoneSignal(&SigDone);
	bIsStarted = true;
	MediaRunnable->Start(EntryFunction, false);
}



// ----------------------------------------------------------------------------
/**
 * Resets the thread to be started again. Must have waited for thread termination using ThreadWaitDone() first!
 */
void FMediaThread::ThreadReset()
{
	check(!bIsStarted || (bIsStarted && SigDone.IsSignaled()));
	if (MediaRunnable)
	{
		FMediaRunnable::Destroy(MediaRunnable);
	}
	MediaRunnable = nullptr;
	bIsStarted = false;
	SigDone.Reset();
}


