// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraBaseModule.h"

#include "CoreTypes.h"
#include "Modules/ModuleManager.h"
#include "IElectraBaseModule.h"

#include "Core/MediaThreads.h"


#define LOCTEXT_NAMESPACE "ElectraBaseModule"

DEFINE_LOG_CATEGORY(LogElectraBase);

// -----------------------------------------------------------------------------------------------------------------------------------

class FElectraBaseModule: public IElectraBaseModule
{
public:
	// IModuleInterface interface

	void StartupModule() override
	{
		if (!bInitialized)
		{
			bInitialized = true;
			FMediaRunnable::Startup();
		}
	}

	void ShutdownModule() override
	{
		if (bInitialized)
		{
			FMediaRunnable::Shutdown();
		}
	}

private:
	bool bInitialized = false;
};

IMPLEMENT_MODULE(FElectraBaseModule, ElectraBase);

#undef LOCTEXT_NAMESPACE


