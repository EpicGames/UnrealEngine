// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParameterDictionary.h"

namespace Electra
{

FVariantValue::FVariantValue()
	: DataType(EDataType::TypeUninitialized)
{
}

FVariantValue::~FVariantValue()
{
	Clear();
}

FVariantValue::FVariantValue(const FVariantValue& rhs)
	: DataType(EDataType::TypeUninitialized)
{
	CopyInternal(rhs);
}

FVariantValue& FVariantValue::operator=(const FVariantValue& rhs)
{
	if (this != &rhs)
	{
		CopyInternal(rhs);
	}
	return *this;
}


FVariantValue::FVariantValue(const FString& StringValue)
	: DataType(EDataType::TypeUninitialized)
{
	Set(StringValue);
}

FVariantValue::FVariantValue(const double DoubleValue)
	: DataType(EDataType::TypeUninitialized)
{
	Set(DoubleValue);
}

FVariantValue::FVariantValue(const int64 Int64Value)
	: DataType(EDataType::TypeUninitialized)
{
	Set(Int64Value);
}

FVariantValue::FVariantValue(const bool BoolValue)
	: DataType(EDataType::TypeUninitialized)
{
	Set(BoolValue);
}

FVariantValue::FVariantValue(const FTimeValue& TimeValue)
	: DataType(EDataType::TypeUninitialized)
{
	Set(TimeValue);
}

FVariantValue::FVariantValue(const FTimespan& TimespanValue)
	: DataType(EDataType::TypeUninitialized)
{
	Set(TimespanValue);
}

FVariantValue::FVariantValue(const FTimecode& TimecodeValue)
	: DataType(EDataType::TypeUninitialized)
{
	Set(TimecodeValue);
}

FVariantValue::FVariantValue(const FFrameRate& FramerateValue)
	: DataType(EDataType::TypeUninitialized)
{
	Set(FramerateValue);
}

FVariantValue::FVariantValue(void* PointerValue)
	: DataType(EDataType::TypeUninitialized)
{
	Set(PointerValue);
}

FVariantValue::FVariantValue(const TArray<uint8>& ArrayValue)
	: DataType(EDataType::TypeUninitialized)
{
	Set(ArrayValue);
}


void FVariantValue::CopyInternal(const FVariantValue& FromOther)
{
	switch(FromOther.DataType)
	{
		case EDataType::TypeUninitialized:
		{
			Clear();
			break;
		}
		case EDataType::TypeFString:
		{
			Set(FromOther.GetFString());
			break;
		}
		case EDataType::TypeDouble:
		{
			Set(FromOther.GetDouble());
			break;
		}
		case EDataType::TypeInt64:
		{
			Set(FromOther.GetInt64());
			break;
		}
		case EDataType::TypeBoolean:
		{
			Set(FromOther.GetBool());
			break;
		}
		case EDataType::TypeTimeValue:
		{
			Set(FromOther.GetTimeValue());
			break;
		}
		case EDataType::TypeTimespanValue:
		{
			Set(FromOther.GetTimespan());
			break;
		}
		case EDataType::TypeTimecodeValue:
		{
			Set(FromOther.GetTimecode());
			break;
		}
		case EDataType::TypeFramerateValue:
		{
			Set(FromOther.GetFramerate());
			break;
		}
		case EDataType::TypeVoidPointer:
		{
			Set(FromOther.GetPointer());
			break;
		}
		case EDataType::TypeSharedPointer:
		{
			Clear();
			FSharedPtrHolderBase* Pointer = reinterpret_cast<FSharedPtrHolderBase*>(&DataBuffer);
			const FSharedPtrHolderBase* OtherPointer = reinterpret_cast<const FSharedPtrHolderBase*>(&FromOther.DataBuffer);
			OtherPointer->SetValueOn(Pointer);
			DataType = FromOther.DataType;
			break;
		}
		case EDataType::TypeU8Array:
		{
			Set(FromOther.GetArray());
			break;
		}
		default:
		{
			Clear();
			check(!"Whoops");
			break;
		}
	}
}

FVariant FVariantValue::ToFVariant() const
{
	switch(GetDataType())
	{
		case EDataType::TypeFString:
		{
			return FVariant(GetFString());
		}
		case EDataType::TypeDouble:
		{
			return FVariant(GetDouble());
		}
		case EDataType::TypeInt64:
		{
			return FVariant(GetInt64());
		}
		case EDataType::TypeBoolean:
		{
			return FVariant(GetBool());
		}
		case EDataType::TypeTimeValue:
		{
			return FVariant(GetTimeValue().GetAsTimespan());
		}
		case EDataType::TypeTimespanValue:
		{
			return FVariant(GetTimespan());
		}
		case EDataType::TypeVoidPointer:
		{
			return FVariant(reinterpret_cast<uint64>(GetPointer()));
		}
		case EDataType::TypeU8Array:
		{
			return FVariant(GetArray());
		}
		// Types that can't be converted.
		case EDataType::TypeTimecodeValue:
		case EDataType::TypeFramerateValue:
		case EDataType::TypeSharedPointer:
		default:
		{
			return FVariant();
		}
	}
}

void FVariantValue::Clear()
{
	switch(DataType)
	{
		case EDataType::TypeFString:
		{
			FString* Str = reinterpret_cast<FString*>(&DataBuffer);
			Str->~FString();
			break;
		}
		case EDataType::TypeTimeValue:
		{
			FTimeValue* Time = reinterpret_cast<FTimeValue*>(&DataBuffer);
			Time->~FTimeValue();
			break;
		}
		case EDataType::TypeTimespanValue:
		{
			FTimespan* Timespan = reinterpret_cast<FTimespan*>(&DataBuffer);
			Timespan->~FTimespan();
			break;
		}
		case EDataType::TypeTimecodeValue:
		{
			FTimecode* Timecode = reinterpret_cast<FTimecode*>(&DataBuffer);
			Timecode->~FTimecode();
			break;
		}
		case EDataType::TypeFramerateValue:
		{
			FFrameRate* Framerate = reinterpret_cast<FFrameRate*>(&DataBuffer);
			Framerate->~FFrameRate();
			break;
		}
		case EDataType::TypeUninitialized:
		case EDataType::TypeDouble:
		case EDataType::TypeInt64:
		case EDataType::TypeBoolean:
		case EDataType::TypeVoidPointer:
		{
			break;
		}
		case EDataType::TypeSharedPointer:
		{
			FSharedPtrHolderBase* Pointer = reinterpret_cast<FSharedPtrHolderBase*>(&DataBuffer);
			Pointer->~FSharedPtrHolderBase();
			break;
		}
		case EDataType::TypeU8Array:
		{
			TArray<uint8>* Array = reinterpret_cast<TArray<uint8>*>(&DataBuffer);
			Array->~TArray<uint8>();
			break;
		}
		default:
		{
			check(!"Whoops");
			break;
		}
	}
	DataType = EDataType::TypeUninitialized;
}


FVariantValue& FVariantValue::Set(const FString& StringValue)
{
	Clear();
	FString* Str = reinterpret_cast<FString*>(&DataBuffer);
	new ((void *)Str) FString(StringValue);
	DataType = EDataType::TypeFString;
	return *this;
}

FVariantValue& FVariantValue::Set(const double DoubleValue)
{
	Clear();
	double* ValuePtr = reinterpret_cast<double*>(&DataBuffer);
	*ValuePtr = DoubleValue;
	DataType = EDataType::TypeDouble;
	return *this;
}

FVariantValue& FVariantValue::Set(const int64 Int64Value)
{
	Clear();
	int64* ValuePtr = reinterpret_cast<int64*>(&DataBuffer);
	*ValuePtr = Int64Value;
	DataType = EDataType::TypeInt64;
	return *this;
}

FVariantValue& FVariantValue::Set(const bool BoolValue)
{
	Clear();
	bool* ValuePtr = reinterpret_cast<bool*>(&DataBuffer);
	*ValuePtr = BoolValue;
	DataType = EDataType::TypeBoolean;
	return *this;
}

FVariantValue& FVariantValue::Set(const FTimeValue& TimeValue)
{
	Clear();
	FTimeValue* ValuePtr = reinterpret_cast<FTimeValue*>(&DataBuffer);
	*ValuePtr = TimeValue;
	DataType = EDataType::TypeTimeValue;
	return *this;
}

FVariantValue& FVariantValue::Set(const FTimespan& TimespanValue)
{
	Clear();
	FTimespan* ValuePtr = reinterpret_cast<FTimespan*>(&DataBuffer);
	*ValuePtr = TimespanValue;
	DataType = EDataType::TypeTimespanValue;
	return *this;
}

FVariantValue& FVariantValue::Set(const FTimecode& TimecodeValue)
{
	Clear();
	FTimecode* ValuePtr = reinterpret_cast<FTimecode*>(&DataBuffer);
	*ValuePtr = TimecodeValue;
	DataType = EDataType::TypeTimecodeValue;
	return *this;
}

FVariantValue& FVariantValue::Set(const FFrameRate& FramerateValue)
{
	Clear();
	FFrameRate* ValuePtr = reinterpret_cast<FFrameRate*>(&DataBuffer);
	*ValuePtr = FramerateValue;
	DataType = EDataType::TypeFramerateValue;
	return *this;
}

FVariantValue& FVariantValue::Set(void* PointerValue)
{
	Clear();
	void** ValuePtr = reinterpret_cast<void**>(&DataBuffer);
	*ValuePtr = PointerValue;
	DataType = EDataType::TypeVoidPointer;
	return *this;
}

FVariantValue& FVariantValue::Set(const TArray<uint8>& ArrayValue)
{
	Clear();
	TArray<uint8>* ValuePtr = reinterpret_cast<TArray<uint8>*>(&DataBuffer);
	new(ValuePtr) TArray<uint8>(ArrayValue);
	DataType = EDataType::TypeU8Array;
	return *this;
}

const FString& FVariantValue::GetFString() const
{
	if (ensure(DataType == EDataType::TypeFString))
	{
		const FString* Str = reinterpret_cast<const FString*>(&DataBuffer);
		return *Str;
	}
	else
	{
		static FString Empty;
		return Empty;
	}
}

const double& FVariantValue::GetDouble() const
{
	if (ensure(DataType == EDataType::TypeDouble))
	{
		const double* Dbl = reinterpret_cast<const double*>(&DataBuffer);
		return *Dbl;
	}
	else
	{
		static double Empty = 0.0;
		return Empty;
	}
}

const int64& FVariantValue::GetInt64() const
{
	if (ensure(DataType == EDataType::TypeInt64))
	{
		const int64* Int = reinterpret_cast<const int64*>(&DataBuffer);
		return *Int;
	}
	else
	{
		static int64 Empty = 0;
		return Empty;
	}
}

const bool& FVariantValue::GetBool() const
{
	if (ensure(DataType == EDataType::TypeBoolean))
	{
		const bool* Bool = reinterpret_cast<const bool*>(&DataBuffer);
		return *Bool;
	}
	else
	{
		static bool Empty = false;
		return Empty;
	}
}

const FTimeValue& FVariantValue::GetTimeValue() const
{
	if (ensure(DataType == EDataType::TypeTimeValue))
	{
		const FTimeValue* Time = reinterpret_cast<const FTimeValue*>(&DataBuffer);
		return *Time;
	}
	else
	{
		static FTimeValue Empty;
		return Empty;
	}
}

const FTimespan& FVariantValue::GetTimespan() const
{
	if (ensure(DataType == EDataType::TypeTimespanValue))
	{
		const FTimespan* Timespan = reinterpret_cast<const FTimespan*>(&DataBuffer);
		return *Timespan;
	}
	else
	{
		static FTimespan Empty;
		return Empty;
	}
}

const FTimecode& FVariantValue::GetTimecode() const
{
	if (ensure(DataType == EDataType::TypeTimecodeValue))
	{
		const FTimecode* Timecode = reinterpret_cast<const FTimecode*>(&DataBuffer);
		return *Timecode;
	}
	else
	{
		static FTimecode Empty;
		return Empty;
	}
}

const FFrameRate& FVariantValue::GetFramerate() const
{
	if (ensure(DataType == EDataType::TypeFramerateValue))
	{
		const FFrameRate* Framerate = reinterpret_cast<const FFrameRate*>(&DataBuffer);
		return *Framerate;
	}
	else
	{
		static FFrameRate Empty;
		return Empty;
	}
}

void* const & FVariantValue::GetPointer() const
{
	if (ensure(DataType == EDataType::TypeVoidPointer))
	{
		void** Pointer = (void**)&DataBuffer;
		return *Pointer;
	}
	else
	{
		static void* Empty = nullptr;
		return Empty;
	}
}

const TArray<uint8>& FVariantValue::GetArray() const
{
	if (ensure(DataType == EDataType::TypeU8Array))
	{
		const TArray<uint8>* Array = reinterpret_cast<const TArray<uint8>*>(&DataBuffer);
		return *Array;
	}
	else
	{
		static TArray<uint8> Empty;
		return Empty;
	}
}


const FString& FVariantValue::SafeGetFString(const FString& Default) const
{
	if (DataType == EDataType::TypeFString)
	{
		const FString* Str = reinterpret_cast<const FString*>(&DataBuffer);
		return *Str;
	}
	return Default;
}

double FVariantValue::SafeGetDouble(double Default) const
{
	if (DataType == EDataType::TypeDouble)
	{
		const double* Dbl = reinterpret_cast<const double*>(&DataBuffer);
		return *Dbl;
	}
	return Default;
}

int64 FVariantValue::SafeGetInt64(int64 Default) const
{
	if (DataType == EDataType::TypeInt64)
	{
		const int64* Int = reinterpret_cast<const int64*>(&DataBuffer);
		return *Int;
	}
	return Default;
}

bool FVariantValue::SafeGetBool(bool Default) const
{
	if (DataType == EDataType::TypeBoolean)
	{
		const bool* Bool = reinterpret_cast<const bool*>(&DataBuffer);
		return *Bool;
	}
	return Default;
}

FTimeValue FVariantValue::SafeGetTimeValue(const FTimeValue& Default) const
{
	if (DataType == EDataType::TypeTimeValue)
	{
		const FTimeValue* Time = reinterpret_cast<const FTimeValue*>(&DataBuffer);
		return *Time;
	}
	return Default;
}

FTimespan FVariantValue::SafeGetTimespan(const FTimespan& Default) const
{
	if (DataType == EDataType::TypeTimespanValue)
	{
		const FTimespan* Timespan = reinterpret_cast<const FTimespan*>(&DataBuffer);
		return *Timespan;
	}
	return Default;
}

FTimecode FVariantValue::SafeGetTimecode(const FTimecode& Default) const
{
	if (DataType == EDataType::TypeTimecodeValue)
	{
		const FTimecode* Timecode = reinterpret_cast<const FTimecode*>(&DataBuffer);
		return *Timecode;
	}
	return Default;
}

FFrameRate FVariantValue::SafeGetFramerate(const FFrameRate& Default) const
{
	if (DataType == EDataType::TypeFramerateValue)
	{
		const FFrameRate* Framerate = reinterpret_cast<const FFrameRate*>(&DataBuffer);
		return *Framerate;
	}
	return Default;
}


void* FVariantValue::SafeGetPointer(void* Default) const
{
	if (DataType == EDataType::TypeVoidPointer)
	{
		void** Pointer = (void**)&DataBuffer;
		return *Pointer;
	}
	return Default;
}

const TArray<uint8>& FVariantValue::SafeGetArray() const
{
	if (DataType == EDataType::TypeU8Array)
	{
		const TArray<uint8>* Array = reinterpret_cast<const TArray<uint8>*>(&DataBuffer);
		return *Array;
	}
	else
	{
		static TArray<uint8> Empty;
		return Empty;
	}
}






FParamDict::FParamDict(const FParamDict& Other)
{
	InternalCopy(Other);
}

FParamDict& FParamDict::operator=(const FParamDict& Other)
{
	if (&Other != this)
	{
		InternalCopy(Other);
	}
	return *this;
}

void FParamDict::InternalCopy(const FParamDict& Other)
{
	Dictionary = Other.Dictionary;
}

void FParamDict::Clear()
{
	Dictionary.Empty();
}

bool FParamDict::HaveKey(const FName& Key) const
{
	return Dictionary.Find(Key) != nullptr;
}

FVariantValue FParamDict::GetValue(const FName& Key) const
{
	static FVariantValue Empty;
	const FVariantValue* VariantValue = Dictionary.Find(Key);
	return VariantValue ? *VariantValue : Empty;
}

void FParamDict::Remove(const FName& Key)
{
	Dictionary.Remove(Key);
}

void FParamDict::Set(const FName& Key, const FVariantValue& Value)
{
	Dictionary.Emplace(Key, Value);
}

void FParamDict::Set(const FName& Key, FVariantValue&& Value)
{
	Dictionary.Emplace(Key, MoveTemp(Value));
}

void FParamDict::GetKeys(TArray<FName>& OutKeys) const
{
	OutKeys.Empty();
	Dictionary.GenerateKeyArray(OutKeys);
}

bool FParamDict::SetValueFrom(FName InKey, const FParamDict& InOther)
{
	FVariantValue OtherValue = InOther.GetValue(InKey);
	const bool bOtherHasKey = OtherValue.IsValid();
	if (bOtherHasKey)
	{
		Set(InKey, MoveTemp(OtherValue));
	}
	return bOtherHasKey;
}

void FParamDict::ConvertKeysStartingWithTo(TMap<FString, FVariant>& OutVariantMap, const FString& InKeyStartsWith, const FString& InAddPrefixToKey) const
{
	OutVariantMap.Reserve(Dictionary.Num());
	FString NewKey;
	NewKey.Reserve(64);
	for(const TPair<FName, FVariantValue>& Pair : Dictionary)
	{
		FString s(Pair.Key.ToString());
		if (!InKeyStartsWith.IsEmpty() && !s.StartsWith(InKeyStartsWith, ESearchCase::CaseSensitive))
		{
			continue;
		}

		NewKey = InAddPrefixToKey;
		NewKey.Append(s);

		FVariant ConvertedValue = Pair.Value.ToFVariant();
		if (!ConvertedValue.IsEmpty())
		{
			OutVariantMap.Emplace(NewKey, MoveTemp(ConvertedValue));
		}
	}
}

void FParamDict::ConvertTo(TMap<FString, FVariant>& OutVariantMap, const FString& InAddPrefixToKey) const
{
	ConvertKeysStartingWithTo(OutVariantMap, FString(), InAddPrefixToKey);
}


} // namespace Electra
