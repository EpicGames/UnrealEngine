// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerTime.h"
#include "Math/BigInt.h"


namespace Electra
{
	FTimeValue& FTimeValue::SetFromTimeFraction(const FTimeFraction& TimeFraction, int64 InSequenceIndex)
	{
		if (TimeFraction.IsValid())
		{
			if (TimeFraction.IsPositiveInfinity())
			{
				SetToPositiveInfinity();
			}
			else
			{
				HNS = TimeFraction.GetAsTimebase(10000000);
				bIsInfinity = false;
				bIsValid = true;
				SequenceIndex = InSequenceIndex;
			}
		}
		else
		{
			SetToInvalid();
		}
		return *this;
	}

	FTimeValue& FTimeValue::SetFromND(int64 Numerator, uint32 Denominator, int64 InSequenceIndex)
	{
		if (Denominator != 0)
		{
			if (Denominator == 10000000)
			{
				HNS 		= Numerator;
				bIsValid	= true;
				bIsInfinity = HNS == 0x7fffffffffffffffLL || HNS == -0x7fffffffffffffffLL;
			}
			else if (Numerator >= -922337203685LL && Numerator <= 922337203685LL)
			{
				HNS 		= Numerator * 10000000 / Denominator;
				bIsValid	= true;
				bIsInfinity = false;
			}
			else
			{
				SetFromTimeFraction(FTimeFraction(Numerator, Denominator));
			}
		}
		else
		{
			HNS 		= Numerator>=0 ? 0x7fffffffffffffffLL : -0x7fffffffffffffffLL;
			bIsValid	= true;
			bIsInfinity = true;
		}
		SequenceIndex = InSequenceIndex;
		return *this;
	}

	//-----------------------------------------------------------------------------
	/**
	 * Returns this time value in a custom timebase. Requires internal bigint conversion and is therefor SLOW!
	 *
	 * @param CustomTimebase
	 *
	 * @return
	 */
	int64 FTimeValue::GetAsTimebase(uint32 CustomTimebase) const
	{
		// Some shortcuts
		if (!bIsValid)
		{
			return 0;
		}
		else if (bIsInfinity)
		{
			return HNS >= 0 ? 0x7fffffffffffffffLL : -0x7fffffffffffffffLL;
		}
		else if (HNS == 0)
		{
			return 0;
		}

		bool bIsNeg = HNS < 0;
		TBigInt<128> n(bIsNeg ? -HNS : HNS);
		TBigInt<128> d(10000000);
		TBigInt<128> s(CustomTimebase);

		n *= s;
		n /= d;

		int64 r = n.ToInt();
		return bIsNeg ? -r : r;
	}
}

