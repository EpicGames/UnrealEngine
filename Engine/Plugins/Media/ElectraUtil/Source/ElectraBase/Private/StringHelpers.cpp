// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/StringHelpers.h"

namespace Electra
{
	namespace StringHelpers
	{

		FString ISO_8859_1_ToFString(const uint8* InStringToConvert, int32 InNumCharsToConvert)
		{
			if (InNumCharsToConvert == 0)
			{
				return FString();
			}
			else if (InNumCharsToConvert < 0)
			{
				for(InNumCharsToConvert=0; InNumCharsToConvert < 16384 && InStringToConvert[InNumCharsToConvert]; ++InNumCharsToConvert)
				{}
			}
			TArray<uint8> ConvBuf;
			ConvBuf.Reserve(InNumCharsToConvert*2);
			for(int32 i=0; i<InNumCharsToConvert; ++i, ++InStringToConvert)
			{
				if (*InStringToConvert == 0x00)
				{
					break;
				}
				if (*InStringToConvert >= 0x20 && *InStringToConvert < 0x7f)
				{
					ConvBuf.Add(*InStringToConvert);
				}
				else  if (*InStringToConvert >= 0xa0)
				{
					// We can convert straight from ISO 8859-1 to UTF8 by doing this:
					ConvBuf.Add(0xc0 | (*InStringToConvert >> 6));
					ConvBuf.Add(0x80 | (*InStringToConvert & 0x3f));
				}
			}
			auto Cnv = StringCast<TCHAR>(reinterpret_cast<const UTF8CHAR*>(ConvBuf.GetData()), ConvBuf.Num());
			return FString::ConstructFromPtrSize(Cnv.Get(), Cnv.Length());
		}

		int32 FindFirstOf(const FString& InString, const FString& SplitAt, int32 FirstPos)
		{
			if (SplitAt.Len() == 1)
			{
				// Speical version for only one split character
				const TCHAR& FindMe = SplitAt[0];
				for (int32 i = FirstPos; i < InString.Len(); ++i)
				{
					if (InString[i] == FindMe)
					{
						return i;
					}
				}
			}
			else
			{
				for (int32 i = FirstPos; i < InString.Len(); ++i)
				{
					for (int32 j = 0; j < SplitAt.Len(); ++j)
					{
						if (InString[i] == SplitAt[j])
						{
							return i;
						}
					}
				}
			}
			return INDEX_NONE;
		}


		int32 FindFirstNotOf(const FString& InString, const FString& InNotOfChars, int32 FirstPos)
		{
			for (int32 i = FirstPos; i < InString.Len(); ++i)
			{
				bool bFoundCharFromNotOfChars = false;
				for (int32 j = 0; j < InNotOfChars.Len(); ++j)
				{
					if (InString[i] == InNotOfChars[j])
					{
						// We found a character from the "NOT of" list. Check next...
						bFoundCharFromNotOfChars = true;
						break;
					}
				}
				if (!bFoundCharFromNotOfChars)
				{
					// We did not find any of the characters. This is what we are looking for and return the index of the first "not of" character
					return i;
				}
			}
			return INDEX_NONE;
		}


		int32 FindLastNotOf(const FString& InString, const FString& InNotOfChars, int32 InStartPos)
		{
			InStartPos = FMath::Min(InStartPos, InString.Len() - 1);
			for (int32 i = InStartPos; i >= 0; --i)
			{
				bool bFoundCharFromNotOfChars = false;
				for (int32 j = 0; j < InNotOfChars.Len(); ++j)
				{
					if (InString[i] == InNotOfChars[j])
					{
						// We found a character from the "NOT of" list. Check next...
						bFoundCharFromNotOfChars = true;
						break;
					}
				}
				if (!bFoundCharFromNotOfChars)
				{
					// We did not find any of the characters. This is what we are looking for and return the index of the first "not of" character
					return i;
				}
			}
			return INDEX_NONE;
		}


		void SplitByDelimiter(TArray<FString>& OutSplits, const FString& InString, const FString& SplitAt)
		{
			if (InString.Len())
			{
				int32 FirstPos = 0;
				while (1)
				{
					int32 SplitPos = InString.Find(SplitAt, ESearchCase::IgnoreCase, ESearchDir::FromStart, FirstPos);
					FString subs = InString.Mid(FirstPos, SplitPos == INDEX_NONE ? MAX_int32 : SplitPos - FirstPos);
					if (subs.Len())
					{
						OutSplits.Push(subs);
					}
					FirstPos = SplitPos + SplitAt.Len();
					if (SplitPos == INDEX_NONE || FirstPos >= InString.Len())
					{
						break;
					}
				}
			}
		}


		bool StringEquals(const TCHAR * const s1, const TCHAR * const s2)
		{
			return FPlatformString::Strcmp(s1, s2) == 0;
		}

		bool StringStartsWith(const TCHAR * const s1, const TCHAR * const s2, SIZE_T n)
		{
			return FPlatformString::Strncmp(s1, s2, n) == 0;
		}

		void StringToArray(TArray<uint8>& OutArray, const FString& InString)
		{
			FTCHARToUTF8 cnv(*InString);
			int32 Len = cnv.Length();
			OutArray.AddUninitialized(Len);
			FMemory::Memcpy(OutArray.GetData(), cnv.Get(), Len);
		}

		FString ArrayToString(const TArray<uint8>& InArray)
		{
			FUTF8ToTCHAR cnv((const ANSICHAR*)InArray.GetData(), InArray.Num());
			FString UTF8Text = FString::ConstructFromPtrSize(cnv.Get(), cnv.Length());
			return MoveTemp(UTF8Text);
		}

		bool ArrayToString(FString& OutString, const TConstArrayView<const uint8>& InArray)
		{
			int32 nb = InArray.Num();
			const uint8* Data = InArray.GetData();
			// Check for potential BOMs
			if (nb > 3 && Data[0] == 0xEF && Data[1] == 0xBB && Data[2] == 0xBF)
			{
				// UTF-8 BOM
				nb -= 3;
				Data += 3;
			}
			else if (nb >= 2 && Data[0] == 0xFE && Data[1] == 0xFF)
			{
				// UTF-16 BE BOM
				return false;
			}
			else if (nb >= 2 && Data[0] == 0xFF && Data[1] == 0xFE)
			{
				// UTF-16 LE BOM
				auto cnv = StringCast<TCHAR>(reinterpret_cast<const UCS2CHAR*>(Data+2), nb/2-1);
				FString UTF8Text = FString::ConstructFromPtrSize(cnv.Get(), cnv.Length());
				OutString = MoveTemp(UTF8Text);
				return true;
			}
			else if (nb >= 4 && Data[0] == 0x00 && Data[1] == 0x00 && Data[2] == 0xFE && Data[3] == 0xFF)
			{
				// UTF-32 BE BOM
				return false;
			}
			else if (nb >= 4 && Data[0] == 0xFF && Data[1] == 0xFE && Data[2] == 0x00 && Data[3] == 0x00)
			{
				// UTF-32 LE BOM
				return false;
			}
			FUTF8ToTCHAR cnv((const ANSICHAR*)Data, nb);
			FString UTF8Text = FString::ConstructFromPtrSize(cnv.Get(), cnv.Length());
			OutString = MoveTemp(UTF8Text);
			return true;
		}


		FString GetLongestCommonPrefix(TArray<FString>& InOutTempArrayOfInputs)
		{
			if (InOutTempArrayOfInputs.IsEmpty())
			{
				return FString();
			}
			else if (InOutTempArrayOfInputs.Num() == 1)
			{
				return InOutTempArrayOfInputs[0];
			}
			InOutTempArrayOfInputs.Sort();
			const int32 l1 =InOutTempArrayOfInputs[0].Len();
			const int32 l2 =InOutTempArrayOfInputs.Last().Len();
			const int32 CommonSize = l1 < l2 ? l1 : l2;
			const TCHAR* s1 = *InOutTempArrayOfInputs[0];
			const TCHAR* s2 = *InOutTempArrayOfInputs.Last();
			int32 NumSame = 0;
			while(NumSame < CommonSize && s1[NumSame] == s2[NumSame])
			{
				++NumSame;
			}
			return FString::ConstructFromPtrSize(s1, NumSame);
		}

	} // namespace StringHelpers
} // namespace Electra
