// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimeFraction.h"
#include "Math/BigInt.h"
#include "Containers/UnrealString.h"

namespace Electra
{

	namespace TimeStringHelpers
	{
		int32 FindFirstNotOf(const FString& InString, const FString& InNotOfChars, int32 FirstPos = 0)
		{
			for (int32 i = FirstPos; i < InString.Len(); ++i)
			{
				bool bFoundCharFromNotOfChars = false;
				for (int32 j = 0; j < InNotOfChars.Len(); ++j)
				{
					if (InString[i] == InNotOfChars[j])
					{
						// We found a character from the "NOT of" list. Check next...
						bFoundCharFromNotOfChars = true;
						break;
					}
				}
				if (!bFoundCharFromNotOfChars)
				{
					// We did not find any of the characters. This is what we are looking for and return the index of the first "not of" character
					return i;
				}
			}
			return INDEX_NONE;
		}

		int32 FindLastNotOf(const FString& InString, const FString& InNotOfChars, int32 InStartPos = MAX_int32)
		{
			InStartPos = FMath::Min(InStartPos, InString.Len() - 1);
			for (int32 i = InStartPos; i >= 0; --i)
			{
				bool bFoundCharFromNotOfChars = false;
				for (int32 j = 0; j < InNotOfChars.Len(); ++j)
				{
					if (InString[i] == InNotOfChars[j])
					{
						// We found a character from the "NOT of" list. Check next...
						bFoundCharFromNotOfChars = true;
						break;
					}
				}
				if (!bFoundCharFromNotOfChars)
				{
					// We did not find any of the characters. This is what we are looking for and return the index of the first "not of" character
					return i;
				}
			}
			return INDEX_NONE;
		}
	}


	FTimeFraction& FTimeFraction::SetFromFloatString(const FString& InString)
	{
		// The string value must consist only of a sign, decimal digits and an optional period.
		static const FString kTextDigitsEtc(TEXT("0123456789.-+"));
		static const FString kTextZero(TEXT("0"));
		if (TimeStringHelpers::FindFirstNotOf(InString, kTextDigitsEtc) == INDEX_NONE)
		{
			Denominator = 1;
			int32 DotIndex;
			InString.FindChar(TCHAR('.'), DotIndex);
			if (DotIndex == INDEX_NONE)
			{
				LexFromString(Numerator, *InString);
				bIsValid = true;
			}
			else
			{
				LexFromString(Numerator, *(InString.Mid(0, DotIndex)));
				FString frc = InString.Mid(DotIndex + 1);
				bool bIsNeg = Numerator < 0;
				if (bIsNeg)
				{
					Numerator = -Numerator;
				}
				// Remove all trailing zeros
				int32 last0 = TimeStringHelpers::FindLastNotOf(frc, kTextZero);
				if (last0 != INDEX_NONE)
				{
					frc.MidInline(0, last0 + 1);
					// Convert at most 7 fractional digits (giving us hundreds of nanoseconds (HNS))
					for(int32 i = 0; i < frc.Len() && i<7; ++i)
					{
						Numerator = Numerator * 10 + (frc[i] - TCHAR('0'));
						Denominator *= 10;
					}
				}
				if (bIsNeg)
				{
					Numerator = -Numerator;
				}
				bIsValid = true;
			}
		}
		else
		{
			static const FString kInf0(TEXT("INF"));
			static const FString kInf1(TEXT("+INF"));
			static const FString kInf2(TEXT("-INF"));
			if (InString.Equals(kInf0, ESearchCase::IgnoreCase) || InString.Equals(kInf1, ESearchCase::IgnoreCase))
			{
				Numerator = 1;
				Denominator = 0;
				bIsValid = true;
			}
			else if (InString.Equals(kInf2, ESearchCase::IgnoreCase))
			{
				Numerator = -1;
				Denominator = 0;
				bIsValid = true;
			}
			else
			{
				bIsValid = false;
			}
		}
		return *this;
	}


	int64 FTimeFraction::GetAsTimebase(uint32 CustomTimebase) const
	{
		// Some shortcuts
		if (!bIsValid)
		{
			return 0;
		}
		else if (CustomTimebase == Denominator)
		{
			return Numerator;
		}
		else if (Numerator == 0)
		{
			return 0;
		}
		// Infinity?
		else if (Denominator == 0 || CustomTimebase == 0)
		{
			return Numerator >= 0 ? 0x7fffffffffffffffLL : -0x7fffffffffffffffLL;
		}

		bool bIsNeg = Numerator < 0;
		TBigInt<128> n(bIsNeg ? -Numerator : Numerator);
		TBigInt<128> d(Denominator);
		TBigInt<128> s(CustomTimebase);

		n *= s;
		n /= d;

		int64 r = n.ToInt();
		return bIsNeg ? -r : r;
	}

	FTimespan FTimeFraction::GetAsTimespan() const
	{
		int64 Ticks = 0;
		if (bIsValid)
		{
			if (Denominator)
			{
				if (Numerator != 0)
				{
					if (Denominator == ETimespan::TicksPerSecond)
					{
						Ticks = Numerator;
					}
					else if (Numerator >= -922337203685LL && Numerator <= 922337203685LL)
					{
						static_assert(ETimespan::TicksPerSecond == 10000000); // otherwise the constants above are wrong
						Ticks = Numerator * ETimespan::TicksPerSecond / Denominator;
					}
					else
					{
						bool bIsNeg = Numerator < 0;
						TBigInt<128> n(bIsNeg ? -Numerator : Numerator);
						TBigInt<128> d(Denominator);
						TBigInt<128> s(ETimespan::TicksPerSecond);
						n *= s;
						n /= d;
						Ticks = n.ToInt();
						if (bIsNeg)
						{
							Ticks = -Ticks;
						}
					}
				}
			}
			else
			{
				Ticks = Numerator < 0 ? ETimespan::MinTicks : ETimespan::MaxTicks;
			}
		}
		return FTimespan(Ticks);
	}

}
