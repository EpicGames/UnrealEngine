// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/TimeUtilities.h"
#include "Utilities/StringHelpers.h"

namespace Electra
{

	namespace
	{
		struct FTimeComponents
		{
			FTimeComponents()
				: Years(0)
				, Months(0)
				, Days(0)
				, Hours(0)
				, Minutes(0)
				, Seconds(0)
				, Milliseconds(0)
				, TimezoneMinutes(0)
			{
			}
			int32		Years;
			int32		Months;
			int32		Days;
			int32		Hours;
			int32		Minutes;
			int32		Seconds;
			int32		Milliseconds;
			int32		TimezoneMinutes;

			bool IsValidUTC() const
			{
				return Years >= 1970 && Years < 2200 /* arbitrary choice in the future */ && Months >= 1 && Months <= 12 && Days >= 1 && Days <= 31 &&
					Hours >= 0 && Hours <= 23 && Minutes >= 0 && Minutes <= 59 && Seconds >= 0 && Seconds <= 60 /*allow for leap second*/ &&
					Milliseconds >= 0 && Milliseconds <= 999 &&
					TimezoneMinutes >= -14 * 60 && TimezoneMinutes <= 14 * 60 /* +/- 14 hours*/;
			}

			static bool IsDigit(const TCHAR c)
			{
				return c >= TCHAR('0') && c <= TCHAR('9');
			}

			static bool IsLeapYear(int32 InYear)
			{
				return (InYear % 4) == 0 && ((InYear % 100) != 0 || (InYear % 400) == 0);
			};

			FTimeValue ToUTC() const
			{
				static const uint8 DaysInMonths[2][12] = { {31, 28, 31, 30, 31, 30, 31, 31, 30, 31, 30, 31}, {31, 29, 31, 30, 31, 30, 31, 31, 30, 31, 30, 31} };
				int64 Sum = 0;
				for (int32 y = 1970; y < Years; ++y)
				{
					Sum += IsLeapYear(y) ? 366 : 365;
				}
				const uint8* const DaysPerMonth = DaysInMonths[IsLeapYear(Years) ? 1 : 0];
				for (int32 m = 1; m < Months; ++m)
				{
					Sum += DaysPerMonth[m - 1];
				}
				Sum += Days - 1;
				Sum *= 24;
				Sum += Hours;
				Sum *= 60;
				Sum += Minutes;
				Sum *= 60;
				Sum += Seconds;
				Sum -= TimezoneMinutes * 60;
				Sum *= 1000;
				Sum += Milliseconds;
				return FTimeValue().SetFromMilliseconds(Sum);
			}
		};


		static int64 ParseSubStringToInt(TCHAR* InFrom, int32 NumChars)
		{
			TCHAR* last = InFrom + NumChars;
			TCHAR c = *last;
			*last = TCHAR('\0');
			int64 v;
			LexFromString(v, InFrom);
			*last = c;
			return v;
		}

	}






	namespace ISO8601
	{

		bool ParseDateTime(FTimeValue& OutTimeValue, const FString& DateTime)
		{
			// Is this a valid date time string in extended format (we require colons and dashes)
			// 		YYYY-MM-DDTHH:MM:SS[.s*][Z]
			if (DateTime.Len() >= 19 && DateTime[10] == TCHAR('T') && DateTime[4] == TCHAR('-') && DateTime[7] == TCHAR('-') && DateTime[13] == TCHAR(':') && DateTime[16] == TCHAR(':'))
			{
				// Make a mutable copy of the string we can alter.
				FString tempString(DateTime);
				TCHAR* tempBuf = GetData(tempString);

				FTimeComponents TimeComponents;

				if (FTimeComponents::IsDigit(tempBuf[0]) && FTimeComponents::IsDigit(tempBuf[1]) && FTimeComponents::IsDigit(tempBuf[2]) && FTimeComponents::IsDigit(tempBuf[3]))
				{
					TimeComponents.Years = ParseSubStringToInt(tempBuf, 4);
					if (FTimeComponents::IsDigit(tempBuf[5]) && FTimeComponents::IsDigit(tempBuf[6]))
					{
						TimeComponents.Months = ParseSubStringToInt(tempBuf + 5, 2);
						if (FTimeComponents::IsDigit(tempBuf[8]) && FTimeComponents::IsDigit(tempBuf[9]))
						{
							TimeComponents.Days = ParseSubStringToInt(tempBuf + 8, 2);
							if (FTimeComponents::IsDigit(tempBuf[11]) && FTimeComponents::IsDigit(tempBuf[12]))
							{
								TimeComponents.Hours = ParseSubStringToInt(tempBuf + 11, 2);
								if (FTimeComponents::IsDigit(tempBuf[14]) && FTimeComponents::IsDigit(tempBuf[15]))
								{
									TimeComponents.Minutes = ParseSubStringToInt(tempBuf + 14, 2);
									if (FTimeComponents::IsDigit(tempBuf[17]) && FTimeComponents::IsDigit(tempBuf[18]))
									{
										TimeComponents.Seconds = ParseSubStringToInt(tempBuf + 17, 2);

										int32 milliSeconds = 0;
										// Are there fractions or timezone?
										TCHAR* Suffix = tempBuf + 19;
										if (Suffix[0] == TCHAR('.'))
										{
											uint32 secFractionalScale = 3;
											for (++Suffix; FTimeComponents::IsDigit(*Suffix); ++Suffix)
											{
												// Only consider 3 fractional digits.
												if (secFractionalScale)
												{
													--secFractionalScale;
													milliSeconds = milliSeconds * 10 + (*Suffix - TCHAR('0'));
												}
											}
											while (secFractionalScale)
											{
												--secFractionalScale;
												milliSeconds *= 10;
											}
										}
										TimeComponents.Milliseconds = milliSeconds;

										// Time zone suffix?
										if (Suffix[0] == TCHAR('+') || Suffix[0] == TCHAR('-'))
										{
											// Offsets have to be [+|-][hh[[:]mm]]
											int32 tzLen = FCString::Strlen(Suffix);
											TCHAR tzSign = Suffix[0];
											if (tzLen >= 3)
											{
												int32 offM = 0;
												int32 offH = ParseSubStringToInt(Suffix + 1, 2);
												Suffix += 3;
												tzLen -= 3;
												// There may or may not be a colon to separate minutes from hours, or no minutes at all.
												if (tzLen)
												{
													if (Suffix[0] == TCHAR(':'))
													{
														Suffix += 1;
														tzLen -= 1;
													}
													// Minutes?
													if (tzLen >= 2)
													{
														offM = ParseSubStringToInt(Suffix, 2);
														Suffix += 2;
													}
												}
												TimeComponents.TimezoneMinutes = offH * 60 + offM;
												if (tzSign == TCHAR('-'))
												{
													TimeComponents.TimezoneMinutes = -TimeComponents.TimezoneMinutes;
												}
											}
											else
											{
												return false;
											}
										}
										else if (Suffix[0] == TCHAR('Z'))
										{
											// We treat the time as UTC in all cases. If it's already that, fine.
											++Suffix;
										}
										if (Suffix[0] != TCHAR('\0'))
										{
											return false;
										}
									}
								}
							}
						}
					}
				}

				if (TimeComponents.IsValidUTC())
				{
					OutTimeValue = TimeComponents.ToUTC();
					return true;
				}
			}
			return false;
		}


		bool ParseDuration(FTimeValue& OutTimeValue, const TCHAR* InDuration)
		{
			// Parse an xs:duration element.
			// We interpret 'years'/'months'/'days' as per the DASH-IF specification where
			//    1 year = 12 months / 1 month = 30 days / 1 day = 24 hours / 1 hour = 60 minutes / 1 minute = 60 seconds.
			// So 1 year is worth only 360 days !!!
			FString tempString(InDuration);
			// See https://www.w3schools.com/xml/schema_dtypes_date.asp
			if ((tempString.Len() && InDuration[0] == TCHAR('P')) || (tempString.Len() > 1 && InDuration[0] == TCHAR('-') && InDuration[1] == TCHAR('P')))
			{
				// Make a mutable copy of the string we can alter.
				TCHAR* tempBuf = GetData(tempString);

				bool bHaveT = false;
				bool bHaveY = false;
				bool bHaveM = false;
				bool bHaveD = false;
				bool bHaveH = false;
				bool bHaveS = false;
				bool bIsNegative = InDuration[0] == TCHAR('-');
				TCHAR* Start = bIsNegative ? tempBuf+2 : tempBuf+1;
				TCHAR* Anchor = nullptr;
				TCHAR* Decimal = nullptr;
				int64 TotalSeconds = 0;
				int64 HNSFraction = 0;
				while(*Start)
				{
					// Skip over consecutive digits until we hit another char
					if (FTimeComponents::IsDigit(*Start))
					{
						if (!Anchor)
						{
							Anchor = Start;
						}
						++Start;
					}
					// Found a 'T' separating year/month/day from hour/minute/second?
					else if (*Start == TCHAR('T'))
					{
						if (!Anchor && !bHaveT)
						{
							bHaveT = true;
							bHaveM = false;
							++Start;
						}
						else
						{
							return false;
						}
					}
					// Years
					else if (*Start == TCHAR('Y'))
					{
						if (Anchor && !bHaveT && !bHaveY)
						{
							int64 Years = ParseSubStringToInt(Anchor, Start-Anchor);
							TotalSeconds += Years * (12 * 30 * 24 * 60 * 60);
							Anchor = nullptr;
							++Start;
							bHaveY = true;
						}
						else
						{
							return false;
						}
					}
					// Months / Minutes
					else if (*Start == TCHAR('M'))
					{
						if (Anchor && !bHaveM)
						{
							int64 Value = ParseSubStringToInt(Anchor, Start-Anchor);
							if (!bHaveT)
							{
								TotalSeconds += Value * (30 * 24 * 60 * 60);
							}
							else
							{
								TotalSeconds += Value * 60;
							}
							Anchor = nullptr;
							++Start;
							bHaveM = true;
						}
						else
						{
							return false;
						}
					}
					// Days
					else if (*Start == TCHAR('D'))
					{
						if (Anchor && !bHaveT && !bHaveD)
						{
							int64 Days = ParseSubStringToInt(Anchor, Start-Anchor);
							TotalSeconds += Days * (24 * 60 * 60);
							Anchor = nullptr;
							++Start;
							bHaveD = true;
						}
						else
						{
							return false;
						}
					}
					// Hours
					else if (*Start == TCHAR('H'))
					{
						if (Anchor && bHaveT && !bHaveH)
						{
							int64 Hours = ParseSubStringToInt(Anchor, Start-Anchor);
							TotalSeconds += Hours * (60 * 60);
							Anchor = nullptr;
							++Start;
							bHaveH = true;
						}
						else
						{
							return false;
						}
					}
					// Seconds
					else if (*Start == TCHAR('S'))
					{
						if (Anchor && bHaveT && !bHaveS)
						{
							// Is this a decimal with fractions?
							if (!Decimal)
							{
								int64 Seconds = ParseSubStringToInt(Anchor, Start-Anchor);
								TotalSeconds += Seconds;
							}
							else
							{
								// First get the full seconds
								int32 nd = Anchor-Decimal-1;
								if (nd > 0)
								{
									int64 Seconds = ParseSubStringToInt(Decimal, nd);
									TotalSeconds += Seconds;
								}
								// Now get the fractional part. We limit it to 7 digits as this is the resolution of an FTimeValue.
								nd = Start - Anchor;
								check(nd > 0);		// This would fire if there are no digits following the decimal point, but then Anchor would be nullptr and we wouldn't even get here.
								if (nd > 7)
								{
									nd = 7;
								}
								HNSFraction = ParseSubStringToInt(Anchor, nd);
								while(++nd <= 7)
								{
									HNSFraction *= 10;
								}
							}
							Anchor = nullptr;
							Decimal = nullptr;
							++Start;
							bHaveS = true;
						}
						else
						{
							return false;
						}
					}
					// Decimal point? We allow both period and comma here.
					else if (*Start == TCHAR('.') || *Start == TCHAR(','))
					{
						// Only allowed for seconds. We don't know what we are parsing yet.
						if (bHaveT && !bHaveS && !Decimal)
						{
							// If there is no digit in front of the decimal point we allow this by setting the decimal to the delimiter.
							Decimal = Anchor ? Anchor : Start;
							Anchor = nullptr;
							++Start;
						}
						else
						{
							return false;
						}
					}
					else
					{
						return false;
					}
				}
				if (Anchor == nullptr && Decimal == nullptr)
				{
					OutTimeValue.SetFromHNS(TotalSeconds * 10000000L + HNSFraction);
					return true;
				}
			}
			return false;
		}


	} // namespace ISO8601


	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/


	namespace RFC7231
	{

		bool ParseDateTime(FTimeValue& OutTimeValue, const FString& DateTime)
		{
			static const TCHAR* const MonthNames[] = { TEXT("Jan"), TEXT("Feb"), TEXT("Mar"), TEXT("Apr"), TEXT("May"), TEXT("Jun"), TEXT("Jul"), TEXT("Aug"), TEXT("Sep"), TEXT("Oct"), TEXT("Nov"), TEXT("Dec") };

			// Three formats need to be considered as per RFC 7231 section 7.1.1.1. :
			//   First the preferred format, if that is no match then obsolete RFC 850 and finally ANSI C asctime()
			// We split the string by space and comma to get 6 groups for IMF, 4 for RFC850 and 5 for asctime.
			TCHAR Groups[6][16];
			const TCHAR* s = GetData(DateTime);
			int32 NumGroups = 0;
			while (*s)
			{
				if (*s == TCHAR(' ') || *s == TCHAR(','))
				{
					++s;
				}
				else
				{
					TCHAR* Group = Groups[NumGroups];
					if (++NumGroups > 6)
					{
						return false;
					}
					int32 GroupLen = 0;
					while (*s && *s != TCHAR(' ') && *s != TCHAR(','))
					{
						*Group++ = *s++;
						// Check that we do not exceed our fixed size array (including one to add terminating NUL char)
						if (++GroupLen == sizeof(Groups[0]) - 1)
						{
							return false;
						}
					}
					// Terminate the group.
					*Group = TCHAR('\0');
				}
			}

			FTimeComponents TimeComponents;

			// Preferred format: IMF-fixdate, ie:  Sun, 06 Nov 1994 08:49:37 GMT
			if (NumGroups == 6)
			{
				TimeComponents.Hours = ParseSubStringToInt(Groups[4], 2);
				TimeComponents.Minutes = ParseSubStringToInt(Groups[4] + 3, 2);
				TimeComponents.Seconds = ParseSubStringToInt(Groups[4] + 6, 2);
				LexFromString(TimeComponents.Days, Groups[1]);
				LexFromString(TimeComponents.Years, Groups[3]);
				for (int32 Month = 0; Month < UE_ARRAY_COUNT(MonthNames); ++Month)
				{
					if (FCString::Strcmp(Groups[2], MonthNames[Month]) == 0)
					{
						TimeComponents.Months = Month + 1;
						break;
					}
				}
			}
			// Obsolete RFC 850 format, ie:  Sunday, 06-Nov-94 08:49:37 GMT
			else if (NumGroups == 4)
			{
				TimeComponents.Hours = ParseSubStringToInt(Groups[2], 2);
				TimeComponents.Minutes = ParseSubStringToInt(Groups[2] + 3, 2);
				TimeComponents.Seconds = ParseSubStringToInt(Groups[2] + 6, 2);

				if (FCString::Strlen(Groups[1]) != 9)
				{
					return false;
				}
				TimeComponents.Days = ParseSubStringToInt(Groups[1], 2);
				TimeComponents.Years = ParseSubStringToInt(Groups[1] + 7, 2);
				// 1970-2069
				TimeComponents.Years += TimeComponents.Years >= 70 ? 1900 : 2000;
				for (int32 Month = 0; Month < UE_ARRAY_COUNT(MonthNames); ++Month)
				{
					if (FCString::Strncmp(Groups[1] + 3, MonthNames[Month], 3) == 0)
					{
						TimeComponents.Months = Month + 1;
						break;
					}
				}
			}
			// ANSI C asctime() format, ie: Sun Nov  6 08:49:37 1994
			else if (NumGroups == 5)
			{
				TimeComponents.Hours = ParseSubStringToInt(Groups[3], 2);
				TimeComponents.Minutes = ParseSubStringToInt(Groups[3] + 3, 2);
				TimeComponents.Seconds = ParseSubStringToInt(Groups[3] + 6, 2);
				LexFromString(TimeComponents.Days, Groups[2]);
				LexFromString(TimeComponents.Years, Groups[4]);
				for (int32 Month = 0; Month < UE_ARRAY_COUNT(MonthNames); ++Month)
				{
					if (FCString::Strcmp(Groups[1], MonthNames[Month]) == 0)
					{
						TimeComponents.Months = Month + 1;
						break;
					}
				}
			}
			else
			{
				return false;
			}

			if (TimeComponents.IsValidUTC())
			{
				OutTimeValue = TimeComponents.ToUTC();
				return true;
			}
			return false;
		}

	} // namespace RFC7231


	namespace RFC2326
	{
		bool ParseNPTTime(FTimeValue& OutTimeValue, const FString& NPTtime)
		{
			// See: https://www.w3.org/TR/media-frags/#naming-time
			// and: https://www.ietf.org/rfc/rfc2326.txt  (section 3.6)
			TArray<FString> NPTparts;
			const TCHAR* const NPTDelimiter = TEXT(":");
			NPTtime.ParseIntoArray(NPTparts, NPTDelimiter, false);
			if (NPTparts.Num() < 4)
			{
				// Note that we do no validation here on whether or not the minutes and seconds are
				// using double digits (in case of h:mm:ss.fff) or if they are within 0-59 range.
				int64 h=0, m=0;
				if (NPTparts.Num() == 3)
				{
					// h:mm:ss[.fff]
					LexFromString(h, *NPTparts[0]);
					LexFromString(m, *NPTparts[1]);
				}
				else if (NPTparts.Num() == 2)
				{
					// mm:ss[.fff]
					LexFromString(m, *NPTparts[1]);
				}
				// The last part will always be seconds with optional fraction. We parse those independently from minutes and hours and add everything.
				OutTimeValue = FTimeValue().SetFromMilliseconds((h * 3600 + m * 60) * 1000) + FTimeValue().SetFromTimeFraction(FTimeFraction().SetFromFloatString(NPTparts.Last()));
				return true;
			}
			return false;
		}
	} // namespace RFC2326



	namespace UnixEpoch
	{
		bool ParseFloatString(FTimeValue& OutTimeValue, const FString& Seconds)
		{
			FTimeFraction t;
			if (t.SetFromFloatString(Seconds).IsValid())
			{
				OutTimeValue.SetFromTimeFraction(t);
				return true;
			}
			return false;
		}
	}


	namespace RFC5905
	{
		bool ParseNTPTime(FTimeValue& OutTimeValue, uint64 NtpTimestampFormat)
		{
			if (NtpTimestampFormat)
			{
				// Parse an NTP time such that it based on the Unix Epoch.
				uint64 Seconds = (NtpTimestampFormat >> 32) - 2208988800UL;			// 70 years, with 17 leap year days as seconds (70*365 + 17) * 86400
				uint64 Nanos = ((NtpTimestampFormat & 0xffffffff) * 1000000) >> 32;
				int64 HNS = Seconds * 10000000 + Nanos * 10;
				OutTimeValue.SetFromHNS(HNS);
				return true;
			}
			return false;
		}
	}


} // namespace Electra


