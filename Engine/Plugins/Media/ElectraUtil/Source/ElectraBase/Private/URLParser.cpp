// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/URLParser.h"


namespace Electra
{
	namespace URLComponents
	{
		static const FString KeepCharsPath(TEXT("/"));
		static const FString KeepCharsFragment(TEXT("!$&'()*+,;=;@/?"));

		static const FString URN(TEXT("urn:"));
		static const FString Data(TEXT("data:"));
		static const FString Slash(TEXT("/"));
		static const FString DotSlash(TEXT("./"));
		static const FString SlashSlash(TEXT("//"));
		static const FString SchemeFile(TEXT("file"));
		static const FString SchemeData(TEXT("data"));
		static const FString SchemeHttp(TEXT("http"));
		static const FString SchemeHttps(TEXT("https"));
		static const FString Colon(TEXT(":"));
		static const FString QuestionMark(TEXT("?"));
		static const FString HashTag(TEXT("#"));
		static const FString Ampersand(TEXT("&"));
		static const FString At(TEXT("@"));
		static const FString BracketOpen(TEXT("["));
		static const FString BracketClose(TEXT("]"));
		static const FString Dot(TEXT("."));
		static const FString DotDot(TEXT(".."));
		static const FString Port80(TEXT("80"));
		static const FString Port443(TEXT("443"));
	}

	static inline void _SwapStrings(FString& a, FString& b)
	{
		FString s = MoveTemp(a);
		a = MoveTemp(b);
		b = MoveTemp(s);
	}

	void FURL_RFC3986::Empty()
	{
		Scheme.Empty();
		UserInfo.Empty();
		Host.Empty();
		Port.Empty();
		Path.Empty();
		Query.Empty();
		Fragment.Empty();
		bIsFile = false;
		bIsData = false;
	}

	void FURL_RFC3986::Swap(FURL_RFC3986& Other)
	{
		_SwapStrings(Scheme, Other.Scheme);
		_SwapStrings(UserInfo, Other.UserInfo);
		_SwapStrings(Host, Other.Host);
		_SwapStrings(Port, Other.Port);
		_SwapStrings(Path, Other.Path);
		_SwapStrings(Query, Other.Query);
		_SwapStrings(Fragment, Other.Fragment);
		bool b = bIsFile; bIsFile = Other.bIsFile; Other.bIsFile = b;
		b = bIsData; bIsData = Other.bIsData; Other.bIsData = b;
	}

	bool FURL_RFC3986::Parse(const FString& InURL)
	{
		Empty();
		if (InURL.IsEmpty())
		{
			return true;
		}
		// Not handling URNs here.
		if (InURL.StartsWith(URLComponents::URN))
		{
			return false;
		}

		// Short circuit data URLs
		if (InURL.StartsWith(URLComponents::Data))
		{
			bIsData = true;
			Scheme = URLComponents::SchemeData;
			Path = InURL.Mid(5);
			return true;
		}

		/*
		RFC 3986, section 3: Syntax components

				 foo://example.com:8042/over/there?name=ferret#nose
				 \_/   \______________/\_________/ \_________/ \__/
				  |           |            |            |        |
			   scheme     authority       path        query   fragment
		*/
		StringHelpers::FStringIterator it(InURL);
		if (InURL.StartsWith(URLComponents::SlashSlash))
		{
			it += 2;
			if (ParseAuthority(it))
			{
				return ParsePathAndQueryFragment(it);
			}
			return false;
		}
		// Check if the URL begins with a path, query or fragment.
		else if (IsPathSeparator(*it) || IsQueryOrFragmentSeparator(*it) || *it == TCHAR('.'))
		{
			return ParsePathAndQueryFragment(it);
		}
		else
		{
			// If we get a URL without a scheme we will trip over the colon separating host and port, mistaking it for
			// the scheme delimiter. For our purposes we only handle URLs that have a scheme when they have an authority!
			FString Component;
			Component.Reserve(InURL.Len());
			// Parse out the first component up to the delimiting colon, path, query or fragment.
			while(it && !IsColonSeparator(*it) && !IsQueryOrFragmentSeparator(*it) && !IsPathSeparator(*it))
			{
				Component += *it++;
			}
			// If we found the colon we assume it separates the scheme from the authority.
			if (it && IsColonSeparator(*it))
			{
				++it;
				if (!it)
				{
					// Cannot just be a scheme with no authority or path.
					return false;
				}
				// Assume we parsed the scheme.
				Scheme = MoveTemp(Component);
				Scheme.ToLowerInline();
				bIsFile = Scheme.Equals(URLComponents::SchemeFile);

				// Does an absolute path ('/') or an authority ('//') follow?
				if (IsPathSeparator(*it))
				{
					++it;
					// Authority?
					if (it && IsPathSeparator(*it))
					{
						++it;

						// Check for a third '/' in Windows filenames like "file:///c:/autoexec.bat"
						if (bIsFile && it.GetRemainingLength() > 3 && IsPathSeparator(*it))
						{
							const TCHAR* Next = it.GetRemainder();
							if (Next[2] == TCHAR(':') && Next[3] == TCHAR('/'))
							{
								++it;
							}
						}

						// Parse out the authority and if successful continue parsing out the path.
						if (!ParseAuthority(it))
						{
							return false;
						}
					}
					else
					{
						// Unwind the absolute path '/' and parse the path.
						--it;
					}
				}
				if (!ParsePathAndQueryFragment(it))
				{
					return false;
				}
			}
			else
			{
				// Got either '/', '?' or '#' while scanning for the scheme. Rewind the iterator and handle the entire thing as a path with or without query or fragment.
				it.Reset();
				return ParsePathAndQueryFragment(it);
			}
		}
		return true;
	}


	void FURL_RFC3986::SetScheme(const FString& InNewScheme)
	{
		Scheme = InNewScheme.ToLower();
		bIsFile = Scheme.Equals(URLComponents::SchemeFile);
		bIsData = Scheme.Equals(URLComponents::SchemeData);
	}
	void FURL_RFC3986::SetHost(const FString& InNewHost)
	{
		Host = InNewHost.ToLower();
	}
	void FURL_RFC3986::SetPort(const FString& InNewPort)
	{
		Port = InNewPort;
	}
	void FURL_RFC3986::SetPath(const FString& InNewPath)
	{
		Path = InNewPath;
	}
	void FURL_RFC3986::SetQuery(const FString& InNewQuery)
	{
		Query = InNewQuery;
	}
	void FURL_RFC3986::SetFragment(const FString& InNewFragment)
	{
		Fragment = InNewFragment;
	}


	FString FURL_RFC3986::GetScheme() const
	{
		return Scheme;
	}

	FString FURL_RFC3986::GetHost() const
	{
		return Host;
	}

	FString FURL_RFC3986::GetPort() const
	{
		return Port;
	}

	FString FURL_RFC3986::GetPath() const
	{
		return Path;
	}

	FString FURL_RFC3986::GetQuery() const
	{
		return Query;
	}

	FString FURL_RFC3986::GetFragment() const
	{
		return Fragment;
	}

	FString FURL_RFC3986::Get(bool bIncludeQuery, bool bIncludeFragment)
	{
		if (bIsData)
		{
			return FString::Printf(TEXT("%s%s"), *URLComponents::Data, *Path);
		}
		FString URL;
		URL.Reserve(Scheme.Len() + UserInfo.Len() + Host.Len() + Port.Len() + Path.Len() + Query.Len() + Fragment.Len() + 32);
		if (IsAbsolute())
		{
			FString Authority = GetAuthority();
			URL = Scheme;
			URL += URLComponents::Colon;
			if (Authority.Len() || bIsFile)
			{
				URL += URLComponents::SlashSlash;
				URL += Authority;
			}
			if (Path.Len())
			{
				if (Authority.Len() && !IsPathSeparator(Path[0]))
				{
					URL += URLComponents::Slash;
				}
				UrlEncode(URL, Path, URLComponents::KeepCharsPath);
			}
			else if ((Query.Len() && bIncludeQuery) || (Fragment.Len() && bIncludeFragment))
			{
				URL += URLComponents::Slash;
			}
		}
		else
		{
			UrlEncode(URL, Path, URLComponents::KeepCharsPath);
		}
		if (Query.Len() && bIncludeQuery)
		{
			URL += URLComponents::QuestionMark;
			URL += Query;
		}
		if (Fragment.Len() && bIncludeFragment)
		{
			URL += URLComponents::HashTag;
			UrlEncode(URL, Fragment, URLComponents::KeepCharsFragment);
		}
		return URL;
	}

	FString FURL_RFC3986::GetPath(bool bIncludeQuery, bool bIncludeFragment)
	{
		if (bIsData)
		{
			return Path;
		}
		FString URL;
		URL.Reserve(Path.Len() + Query.Len() + Fragment.Len() + 32);
		if (IsAbsolute())
		{
			if (Path.Len())
			{
				if (GetAuthority().Len() && !IsPathSeparator(Path[0]))
				{
					URL += URLComponents::Slash;
				}
				UrlEncode(URL, Path, URLComponents::KeepCharsPath);
			}
			else if ((Query.Len() && bIncludeQuery) || (Fragment.Len() && bIncludeFragment))
			{
				URL += URLComponents::Slash;
			}
		}
		else
		{
			UrlEncode(URL, Path, URLComponents::KeepCharsPath);
		}
		if (Query.Len() && bIncludeQuery)
		{
			URL += URLComponents::QuestionMark;
			URL += Query;
		}
		if (Fragment.Len() && bIncludeFragment)
		{
			URL += URLComponents::HashTag;
			UrlEncode(URL, Fragment, URLComponents::KeepCharsFragment);
		}
		return URL;
	}

	void FURL_RFC3986::GetQueryParams(TArray<FQueryParam>& OutQueryParams, const FString& InQueryParameters, bool bPerformUrlDecoding, bool bSameNameReplacesValue)
	{
		TArray<FString> ValuePairs;
		InQueryParameters.ParseIntoArray(ValuePairs, *URLComponents::Ampersand, true);
		for(int32 i=0; i<ValuePairs.Num(); ++i)
		{
			int32 EqPos = 0;
			ValuePairs[i].FindChar(TCHAR('='), EqPos);
			FQueryParam qp;
			bool bOk = true;
			FString k, v;
			if (EqPos != INDEX_NONE)
			{
				k = ValuePairs[i].Mid(0, EqPos);
				v = ValuePairs[i].Mid(EqPos+1);
			}
			else
			{
				k = ValuePairs[i];
			}

			if (bPerformUrlDecoding)
			{
				bOk = UrlDecode(qp.Name, k) && UrlDecode(qp.Value, v);
			}
			else
			{
				qp.Name = MoveTemp(k);
				qp.Value = MoveTemp(v);
			}
			if (bOk)
			{
				if (!bSameNameReplacesValue)
				{
					OutQueryParams.Emplace(MoveTemp(qp));
				}
				else
				{
					bool bFound = false;
					for(int32 j=0; j<OutQueryParams.Num(); ++j)
					{
						if (OutQueryParams[j].Name.Equals(qp.Name))
						{
							OutQueryParams[j] = MoveTemp(qp);
							bFound = true;
							break;
						}
					}
					if (!bFound)
					{
						OutQueryParams.Emplace(MoveTemp(qp));
					}
				}
			}
		}
	}

	void FURL_RFC3986::GetQueryParams(TArray<FQueryParam>& OutQueryParams, bool bPerformUrlDecoding, bool bSameNameReplacesValue)
	{
		GetQueryParams(OutQueryParams, Query, bPerformUrlDecoding, bSameNameReplacesValue);
	}

	void FURL_RFC3986::SetQueryParams(const TArray<FQueryParam>& InQueryParams)
	{
		Query.Empty();
		if (bIsData)
		{
			return;
		}

		int32 Len=0;
		for(int32 i=0; i<InQueryParams.Num(); ++i)
		{
			Len += InQueryParams[i].Name.Len() + InQueryParams[i].Value.Len() + 2;
		}
		Query.Reserve(Len);
		for(int32 i=0; i<InQueryParams.Num(); ++i)
		{
			Query += InQueryParams[i].Name + '=' + InQueryParams[i].Value;
			if (i != InQueryParams.Num()-1)
			{
				Query.AppendChar(TCHAR('&'));
			}
		}
	}

	// Appends or prepends additional query parameters.
    void FURL_RFC3986::AddOrUpdateQueryParams(const TArray<FQueryParam>& InQueryParams)
	{
		if (bIsData)
		{
			return;
		}
		TArray<FQueryParam> Current;
		GetQueryParams(Current, false, true);
		for(int32 i=0; i<InQueryParams.Num(); ++i)
		{
			bool bUpdated = false;
			for(int32 j=0; j<Current.Num(); ++j)
			{
				if (Current[j].Name.Equals(InQueryParams[i].Name))
				{
					Current[j].Value = InQueryParams[i].Value;
					bUpdated = true;
					break;
				}
			}
			if (!bUpdated)
			{
				Current.Emplace(InQueryParams[i]);
			}
		}
		SetQueryParams(Current);
	}

	void FURL_RFC3986::AddOrUpdateQueryParams(const FString& InQueryParameters)
	{
		if (InQueryParameters.Len() > 0)
		{
			FString _Query(InQueryParameters);
			if (_Query[0] == TCHAR('?') || _Query[0] == TCHAR('&'))
			{
				_Query.RightChopInline(1, EAllowShrinking::No);
			}
			if (_Query.Len() && _Query[_Query.Len()-1] == TCHAR('&'))
			{
				_Query.LeftChopInline(1, EAllowShrinking::No);
			}
			TArray<FQueryParam> qp;
			GetQueryParams(qp, _Query, false, true);
			AddOrUpdateQueryParams(qp);
		}
	}

	bool FURL_RFC3986::IsAbsolute() const
	{
		return Scheme.Len() > 0;
	}

	bool FURL_RFC3986::GetPathComponents(TArray<FString>& OutPathComponents) const
	{
		if (bIsData)
		{
			int32 CommaPos = INDEX_NONE;
			if (Path.FindChar(TCHAR(','), CommaPos))
			{
				FString MediaType = Path.Mid(0, CommaPos);
				MediaType.ParseIntoArray(OutPathComponents, TEXT(";"), true);
				OutPathComponents.Emplace(Path.Mid(CommaPos + 1));
				return true;
			}
			return false;
		}
		GetPathComponents(OutPathComponents, GetPath());
		return true;
	}

	FString FURL_RFC3986::GetLastPathComponent() const
	{
		TArray<FString> Components;
		GetPathComponents(Components, GetPath());
		return Components.Num() ? Components.Last() : FString();
	}

	FString FURL_RFC3986::GetAuthority() const
	{
		FString Authority;
		Authority.Reserve(Scheme.Len() + UserInfo.Len() + Host.Len() + Port.Len() + 32);
		if (UserInfo.Len())
		{
			Authority += UserInfo;
			Authority += URLComponents::At;
		}
		if (bIsFile)
		{
			Authority += Host;
		}
		else
		{
			// Is the host an IPv6 address that needs to be enclosed in square brackets?
			int32 DummyPos = 0;
			if (Host.FindChar(TCHAR(':'), DummyPos))
			{
				Authority += URLComponents::BracketOpen;
				Authority += Host;
				Authority += URLComponents::BracketClose;
			}
			else
			{
				Authority += Host;
			}
			// Need to append a port?
			if (Port.Len())
			{
				Authority += URLComponents::Colon;
				Authority += Port;
			}
		}
		return Authority;
	}

	bool FURL_RFC3986::HasSameOriginAs(const FURL_RFC3986& Other)
	{
		/* RFC 6454:
			5.  Comparing Origins

			   Two origins are "the same" if, and only if, they are identical.  In
			   particular:

			   o  If the two origins are scheme/host/port triples, the two origins
				  are the same if, and only if, they have identical schemes, hosts,
				  and ports.
		*/
		return Scheme.Equals(Other.Scheme) && Host.Equals(Other.Host) && Port.Equals(Other.Port);
	}

	bool FURL_RFC3986::ParseAuthority(StringHelpers::FStringIterator& it)
	{
		UserInfo.Empty();
		FString Component;
		Component.Reserve(it.GetRemainingLength());
		while(it && !IsPathSeparator(*it) && !IsQueryOrFragmentSeparator(*it))
		{
			// If there is a user-info delimiter?
			if (*it == TCHAR('@'))
			{
				// Yes, what we have gathered so far is the user info. Set it and gather from scratch.
				UserInfo = Component;
				Component.Empty();
			}
			else
			{
				Component += *it;
			}
			++it;
		}
		StringHelpers::FStringIterator SubIt(Component);
		return ParseHostAndPort(SubIt);
	}

	bool FURL_RFC3986::ParseHostAndPort(StringHelpers::FStringIterator& it)
	{
		if (!it)
		{
			return true;
		}
		Host.Empty();
		Host.Reserve(it.GetRemainingLength());
		if (bIsFile)
		{
			// For file:// scheme we have to consider Windows drive letters with a colon, eg. file://D:/
			// We must not stop parsing at the colon since it will not indicate a port for the file scheme anyway.
			while(it)
			{
				Host += *it++;
			}
		}
		else
		{
			// IPv6 adress in [xxx:xxx:xxx] notation?
			if (*it == TCHAR('['))
			{
				++it;
				while(it && *it != TCHAR(']'))
				{
					Host += *it++;
				}
				if (!it)
				{
					// Need to have at least the closing ']'
					return false;
				}
				++it;
			}
			else
			{
				while(it && !IsColonSeparator(*it))
				{
					Host += *it++;
				}
			}
			if (it && IsColonSeparator(*it))
			{
				++it;
				Port.Empty();
				Port.Reserve(16);
				while(it)
				{
					Port += *it++;
				}
			}
			Host.ToLowerInline();
		}
		return true;
	}

	bool FURL_RFC3986::ParsePathAndQueryFragment(StringHelpers::FStringIterator& it)
	{
		if (it)
		{
			if (!IsQueryOrFragmentSeparator(*it))
			{
				if (!ParsePath(it))
				{
					return false;
				}
			}
			if (it && IsQuerySeparator(*it))
			{
				if (!ParseQuery(++it))
				{
					return false;
				}
			}
			if (it && IsFragmentSeparator(*it))
			{
				return ParseFragment(++it);
			}
		}
		return true;
	}

	bool FURL_RFC3986::ParsePath(StringHelpers::FStringIterator& it)
	{
		Path.Empty();
		Path.Reserve(it.GetRemainingLength());
		while(it && !IsQueryOrFragmentSeparator(*it))
		{
			Path += *it++;
		}
		// URL decode the path internally.
		FString Decoded;
		if (UrlDecode(Decoded, Path))
		{
			Path = Decoded;
			return true;
		}
		return false;
	}

	bool FURL_RFC3986::ParseQuery(StringHelpers::FStringIterator& it)
	{
		Query.Empty();
		Query.Reserve(it.GetRemainingLength());
		// Query is all up to the end or the start of the fragment.
		while(it && !IsFragmentSeparator(*it))
		{
			Query += *it++;
		}
		return true;
	}

	bool FURL_RFC3986::ParseFragment(StringHelpers::FStringIterator& it)
	{
		// Fragment being the last part of the URL extends all the way to the end.
		Fragment = it.GetRemainder();
		it.SetToEnd();
		// URL decode the fragment internally.
		FString Decoded;
		if (UrlDecode(Decoded, Fragment))
		{
			Fragment = Decoded;
			return true;
		}
		return false;
	}

	bool FURL_RFC3986::UrlDecode(FString& OutResult, const FString& InUrlToDecode)
	{
		TArray<uint8> AsciiString;
		AsciiString.Reserve(InUrlToDecode.Len() + 1);
		auto IsHexChar = [](ANSICHAR In) ->bool
		{
			return (In >= ANSICHAR('0') && In <= ANSICHAR('9')) || (In >= ANSICHAR('a') && In <= ANSICHAR('f')) || (In >= ANSICHAR('A') && In <= ANSICHAR('F'));
		};
		auto HexVal = [](ANSICHAR In) -> uint8
		{
			if (In >= ANSICHAR('0') && In <= ANSICHAR('9'))
			{
				return In - ANSICHAR('0');
			}
			else if (In >= ANSICHAR('a') && In <= ANSICHAR('f'))
			{
				return In - ANSICHAR('a') + 10;
			}
			else
			{
				return In - ANSICHAR('A') + 10;
			}
		};
		// The URL to be decoded should strictly speaking consist only of ASCII characters, but depending on how this is used
		// it may as well be a UTF8 encoded string. We decode UTF8 to ASCII, which is not doing anything if the URL is already
		// only composed of ASCII characters, then we unescape it and finally convert it back to UTF8, which we need to do
		// to get any UTF8 characters back that are represented in the URL through escaping.
		FTCHARToUTF8 UTF8String(*InUrlToDecode);
		for(const ANSICHAR* InUTF8Bytes=UTF8String.Get(); *InUTF8Bytes; ++InUTF8Bytes)
		{
			// Escaped?
			if (*InUTF8Bytes == ANSICHAR('%') && InUTF8Bytes[1] && IsHexChar(InUTF8Bytes[1]) && InUTF8Bytes[2] && IsHexChar(InUTF8Bytes[2]))
			{
				uint8 hi = HexVal(*(++InUTF8Bytes));
				uint8 lo = HexVal(*(++InUTF8Bytes));
				AsciiString.Add((uint8)(hi * 16 + lo));
			}
			else
			{
				AsciiString.Add((uint8)*InUTF8Bytes);
			}
		}
		AsciiString.Add((uint8)0);
		// Convert to UTF8 to get any escaped sequences back to what they were.
		OutResult = UTF8_TO_TCHAR((const ANSICHAR*)AsciiString.GetData());
		return true;
	}

	bool FURL_RFC3986::UrlEncode(FString& OutResult, const FString& InUrlToEncode, const FString& InCharsToKeep)
	{
		auto AnsiString = StringCast<ANSICHAR>(*InCharsToKeep);
		const ANSICHAR* InKeepCharsASCII = AnsiString.Get();
		auto KeepUnchanged = [InKeepCharsASCII](ANSICHAR In) -> bool
		{
			for(const ANSICHAR* Res=InKeepCharsASCII; *Res; ++Res)
			{
				if (*Res == In)
				{
					return true;
				}
			}
			return false;
		};
		auto IsHexChar = [](ANSICHAR In) ->bool
		{
			return (In >= ANSICHAR('0') && In <= ANSICHAR('9')) || (In >= ANSICHAR('a') && In <= ANSICHAR('f')) || (In >= ANSICHAR('A') && In <= ANSICHAR('F'));
		};

		OutResult.Reserve(InUrlToEncode.Len() * 3);	// assume we need to encode every character
		FTCHARToUTF8 UTF8String(*InUrlToEncode);
		for(const ANSICHAR* InUTF8Bytes=UTF8String.Get(); *InUTF8Bytes; ++InUTF8Bytes)
		{
			uint8 c = (uint8)(*InUTF8Bytes);
			// Unreserved character?
			if ((c >= uint8('a') && c <= uint8('z')) || (c >= uint8('0') && c <= uint8('9')) || (c >= uint8('A') && c <= uint8('Z')) || c == uint8('-') || c == uint8('_') || c == uint8('.') || c == uint8('~'))
			{
				OutResult += TCHAR(c);
			}
			// Already escaped?
			else if (c == uint8('%') && InUTF8Bytes[1] && IsHexChar(InUTF8Bytes[1]) && InUTF8Bytes[2] && IsHexChar(InUTF8Bytes[2]))
			{
				OutResult += TCHAR(c);
				OutResult += TCHAR(*(++InUTF8Bytes));
				OutResult += TCHAR(*(++InUTF8Bytes));
			}
			else if (KeepUnchanged(*InUTF8Bytes))
			{
				OutResult += TCHAR(c);
			}
			else
			{
				OutResult += FString::Printf(TEXT("%%%02X"), c);
			}
		}
		return true;
	}

	FString FURL_RFC3986::GetStandardPortForScheme(const FString& InScheme, bool bIgnoreCase)
	{
		if (InScheme.Equals(URLComponents::SchemeHttp, bIgnoreCase ? ESearchCase::IgnoreCase : ESearchCase::CaseSensitive))
		{
			return URLComponents::Port80;
		}
		else if (InScheme.Equals(URLComponents::SchemeHttps, bIgnoreCase ? ESearchCase::IgnoreCase : ESearchCase::CaseSensitive))
		{
			return URLComponents::Port443;
		}
		return FString();
	}


	FString FURL_RFC3986::GetUrlEncodeSubDelimsChars()
	{
		return FString(TEXT("!$&'()*+,;="));
	}

	void FURL_RFC3986::GetPathComponents(TArray<FString>& OutPathComponents, const FString& InPath)
	{
		TArray<FString> Components;
		// Split on '/', ignoring resulting empty parts (eg. "a//b" gives ["a", "b"] only instead of ["a", "", "b"] !!
		InPath.ParseIntoArray(Components, *URLComponents::Slash, true);
		OutPathComponents.Append(Components);
	}

	FURL_RFC3986& FURL_RFC3986::ResolveWith(const FString& InChildURL)
	{
		FURL_RFC3986 Other;
		if (Other.Parse(InChildURL))
		{
			ResolveWith(Other);
		}
		return *this;
	}

	FURL_RFC3986& FURL_RFC3986::ResolveAgainst(const FString& InParentURL)
	{
		FURL_RFC3986 Parent;
		if (Parent.Parse(InParentURL))
		{
			Parent.ResolveWith(*this);
			Swap(Parent);
		}
		return *this;
	}

	/**
	 * Resolve URL as per RFC 3986 section 5.2
	 */
	void FURL_RFC3986::ResolveWith(const FURL_RFC3986& Other)
	{
		// data urls cannot be resolved, but we handle them here for consistency.
		if (Other.bIsData)
		{
			Empty();
			Scheme = Other.Scheme;
			Path = Other.Path;
		}
		else if (bIsData && Other.Scheme.IsEmpty())
		{
			// Nothing to do.
			return;
		}
		else if (!Other.Scheme.IsEmpty())
		{
			Scheme   = Other.Scheme;
			UserInfo = Other.UserInfo;
			Host     = Other.Host;
			Port     = Other.Port;
			Path     = Other.Path;
			Query    = Other.Query;
			RemoveDotSegments();
		}
		else
		{
			if (!Other.Host.IsEmpty())
			{
				UserInfo = Other.UserInfo;
				Host     = Other.Host;
				Port     = Other.Port;
				Path     = Other.Path;
				Query    = Other.Query;
				RemoveDotSegments();
			}
			else
			{
				if (Other.Path.IsEmpty())
				{
					if (!Other.Query.IsEmpty())
					{
						Query = Other.Query;
					}
				}
				else
				{
					if (IsPathSeparator(Other.Path[0]))
					{
						Path = Other.Path;
					}
					else
					{
						MergePath(Other.Path);
					}
					RemoveDotSegments();
					Query = Other.Query;
				}
			}
		}
		Fragment = Other.Fragment;
		bIsFile = Scheme.Equals(URLComponents::SchemeFile);
		bIsData = Scheme.Equals(URLComponents::SchemeData);
	}

	void FURL_RFC3986::BuildPathFromSegments(const TArray<FString>& Components, bool bAddLeadingSlash, bool bAddTrailingSlash)
	{
		Path.Empty();
		int32 ComponentLen = 0;
		for(int32 i=0, iMax=Components.Num(); i<iMax; ++i)
		{
			ComponentLen += Components[i].Len() + 1;
		}
		Path.Reserve(ComponentLen + 8);
		int32 DummyPos = 0;
		for(int32 i=0, iMax=Components.Num(); i<iMax; ++i)
		{
			if (i == 0)
			{
				if (bAddLeadingSlash)
				{
					Path = URLComponents::Slash;
				}
				/*
					As per RFC 3986 Section 4.2 Relative Reference:
						A path segment that contains a colon character (e.g., "this:that")
						cannot be used as the first segment of a relative-path reference, as
						it would be mistaken for a scheme name.  Such a segment must be
						preceded by a dot-segment (e.g., "./this:that") to make a relative-
						path reference.
				*/
				else if (Scheme.IsEmpty() && Components[0].FindChar(TCHAR(':'), DummyPos))
				{
					Path = URLComponents::DotSlash;
				}
			}
			else
			{
				Path += TCHAR('/');
			}
			Path += Components[i];
		}
		if (bAddTrailingSlash)
		{
			Path += TCHAR('/');
		}
	}

	void FURL_RFC3986::MergePath(const FString& InPathToMerge)
	{
		// RFC 3986 Section 5.2.3. Merge Paths
		if (Host.Len() && Path.IsEmpty())
		{
			Path.Reserve(InPathToMerge.Len() + 1);
			Path = URLComponents::Slash;
			Path += InPathToMerge;
		}
		else
		{
			int32 LastSlashPos = INDEX_NONE;
			// Is there a '/' somewhere in the current path that's not at the end?
			if (Path.FindLastChar(TCHAR('/'), LastSlashPos))
			{
				if (LastSlashPos != Path.Len()-1)
				{
					Path.LeftInline(LastSlashPos+1);
				}
				Path += InPathToMerge;
			}
			else
			{
				Path = InPathToMerge;
			}
		}
	}

	void FURL_RFC3986::RemoveDotSegments()
	{
		if (Path.Len())
		{
			TArray<FString> NewSegments;
			// Remember the current path starting or ending in a slash.
			bool bHasLeadingSlash = IsPathSeparator(Path[0]);
			bool bHasTrailingSlash = Path.EndsWith(URLComponents::Slash);
			// Split the path into its segments
			TArray<FString> Segments;
			GetPathComponents(Segments);
			for(int32 i=0; i<Segments.Num(); ++i)
			{
				// For every ".." go up one level.
				if (Segments[i].Equals(URLComponents::DotDot))
				{
					if (NewSegments.Num())
					{
						NewSegments.Pop();
					}
				}
				// Add non- "." segments.
				else if (!Segments[i].Equals(URLComponents::Dot))
				{
					NewSegments.Add(Segments[i]);
				}
			}
			BuildPathFromSegments(NewSegments, bHasLeadingSlash, bHasTrailingSlash);
		}
	}


	bool FURL_RFC3986::MakeRelativePath(FString& OutRelativePath, const FString& InAbsoluteURLToMakeRelativeToThis) const
	{
		FURL_RFC3986 Other;
		if (!Other.Parse(InAbsoluteURLToMakeRelativeToThis))
		{
			return false;
		}
		// Scheme, user info, host and port have to match.
		if (!Scheme.Equals(Other.Scheme) || !UserInfo.Equals(Other.UserInfo) || !Host.Equals(Other.Host) || !Port.Equals(Other.Port))
		{
			return false;
		}
		// Both paths must not end on a slash
		if (Path.EndsWith(URLComponents::Slash) || Other.Path.EndsWith(URLComponents::Slash))
		{
			return false;
		}
		// Split the paths into arrays
		TArray<FString> ThisPaths, OtherPaths;
		GetPathComponents(ThisPaths, Path);
		GetPathComponents(OtherPaths, Other.Path);
		while(ThisPaths.Num() && OtherPaths.Num() && ThisPaths[0].Equals(OtherPaths[0]))
		{
			ThisPaths.RemoveAt(0);
			OtherPaths.RemoveAt(0);
		}

		auto AssemblePaths = [](const TArray<FString>& InPaths) -> FString
		{
			FString p;
			for(int32 i=0,iMax=InPaths.Num(); i<iMax; ++i)
			{
				if (i)
				{
					p.Append(URLComponents::Slash);
				}
				p.Append(InPaths[i]);
			}
			return p;
		};

		FString RelPath;
		// If this path is now only the filename, the other is relative to here.
		if (ThisPaths.Num() == 1)
		{
			RelPath = TEXT("./");
		}
		else
		{
			for(int32 i=1; i<ThisPaths.Num(); ++i)
			{
				RelPath.Append(TEXT("../"));
			}
		}
		RelPath.Append(AssemblePaths(OtherPaths));

		if (Other.Query.Len())
		{
			RelPath += URLComponents::QuestionMark;
			RelPath += Other.Query;
		}
		if (Other.Fragment.Len())
		{
			RelPath += URLComponents::HashTag;
			UrlEncode(RelPath, Other.Fragment, URLComponents::KeepCharsFragment);
		}
		OutRelativePath = MoveTemp(RelPath);
		return true;
	}


} // namespace Electra
