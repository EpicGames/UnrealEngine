// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/BCP47-Helpers.h"
#include "Utilities/ISO639-Map.h"
#include "Internationalization/Regex.h"

namespace Electra
{
	namespace BCP47
	{
		/*
			Note: If, at some point, we wanted to canonicalize the language tag we can use the
			      IANA database as a source of information
					https://www.iana.org/assignments/language-subtag-registry/language-subtag-registry

				  The format is explained in RFC-5646.
		*/

		static const TArray<const FString> Irregulars
		{
			TEXT("en-GB-oed"),
			TEXT("i-ami"),
			TEXT("i-bnn"),
			TEXT("i-default"),
			TEXT("i-enochian"),
			TEXT("i-hak"),
			TEXT("i-klingon"),
			TEXT("i-lux"),
			TEXT("i-mingo"),
			TEXT("i-navajo"),
			TEXT("i-pwn"),
			TEXT("i-tao"),
			TEXT("i-tay"),
			TEXT("i-tsu"),
			TEXT("sgn-BE-FR"),
			TEXT("sgn-BE-NL"),
			TEXT("sgn-CH-DE")
		};

		static const TArray<const FString> Regulars
		{
			TEXT("art-lojban"),
			TEXT("cel-gaulish"),
			TEXT("no-bok"),
			TEXT("no-nyn"),
			TEXT("zh-guoyu"),
			TEXT("zh-hakka"),
			TEXT("zh-min"),
			TEXT("zh-min-nan"),
			TEXT("zh-xiang")
		};

		namespace Regexes
		{
			static const FRegexPattern& WholePrivateUse()
			{
				static const FRegexPattern pu(TEXT(R"(^([xX](?:-[a-zA-Z0-9]{1,8})+)$)"));
				return pu;
			}
			static const FRegexPattern& Language()
			{
				static const FRegexPattern la(TEXT(R"(^((?:(?:[a-zA-Z]{2,3}(?:(?:-[a-zA-Z]{3}){0,3})|(?:[a-zA-Z]{4})|(?:[a-zA-Z]{5,8}))(?=-|$))))"));
				return la;
			}
			static const FRegexPattern& Script()
			{
				static const FRegexPattern sc(TEXT(R"(^-([a-zA-Z]{4})(?=-|$))"));
				return sc;
			}
			static const FRegexPattern& Region()
			{
				static const FRegexPattern re(TEXT(R"(^-([a-zA-Z]{2}|[0-9]{3})(?=-|$))"));
				return re;
			}
			static const FRegexPattern& Variant()
			{
				static const FRegexPattern va(TEXT(R"(^-([a-zA-Z0-9]{5,8}(?=-|$)|(?:[0-9][a-zA-Z0-9]{3}(?=-|$))))"));
				return va;
			}
			static const FRegexPattern& Extension()
			{
				static const FRegexPattern ex(TEXT(R"(^-([0-9a-wyzA-WYZ](?:(?:-[a-zA-Z0-9]{2,8})+)(?=-|$)))"));
				return ex;
			}
			static const FRegexPattern& PrivateUse()
			{
				static const FRegexPattern pu(TEXT(R"(^-([xX](?:-[a-zA-Z0-9]{1,8})+))"));
				return pu;
			}
		}


		static bool ParseInternal(FLanguageTag& OutTag, FString& OutError, const FString& InRFC5646)
		{
			// First check if this is a private use tag as a whole
			FRegexMatcher WholePrivateUse(Regexes::WholePrivateUse(), InRFC5646);
			if (WholePrivateUse.FindNext())
			{
				OutTag.FullLanguage = OutTag.PrimaryLanguage = WholePrivateUse.GetCaptureGroup(1);
				return true;
			}
			// Then check if it is a grandfathered tag.
			auto CheckGrandfathered = [](FLanguageTag& Out, const FString& InTag, const TArray<const FString>& InList) -> bool
			{
				for(auto& i : InList)
				{
					// If it matches set the language from the list so the capitalization is as it should be,.
					if (i.Equals(InTag, ESearchCase::IgnoreCase))
					{
						Out.FullLanguage = Out.PrimaryLanguage = i;
						return true;
					}
				}
				return false;
			};
			if (CheckGrandfathered(OutTag, InRFC5646, Irregulars) || CheckGrandfathered(OutTag, InRFC5646, Regulars))
			{
				return true;
			}

			enum class ESubPart
			{
				None,
				Language,
				Script,
				Region,
				Variant,
				Extension,
				PrivateUse
			};
			static const TCHAR* const SubPartNames[] =
			{
				TEXT("none"),
				TEXT("language"),
				TEXT("script"),
				TEXT("region"),
				TEXT("variant"),
				TEXT("extension"),
				TEXT("privateuse")
			};
			ESubPart LastSuccessfulSubPart = ESubPart::None;
			int32 ParsePos = 0;
			FString Remainder(InRFC5646);
			FRegexMatcher Language(Regexes::Language(), Remainder);
			if (!Language.FindNext())
			{
				OutError = TEXT("Language not found at beginning");
				return false;
			}
			OutTag.FullLanguage = Language.GetCaptureGroup(1);
			if (OutTag.FullLanguage.Len() == 4)
			{
				OutError = TEXT("Four letter language is reserved for future use");
				return false;
			}
			Remainder.MidInline(OutTag.FullLanguage.Len());
			ParsePos += OutTag.FullLanguage.Len();
			LastSuccessfulSubPart = ESubPart::Language;
			int32 PrimLangPos;
			if (OutTag.FullLanguage.FindChar(TCHAR('-'), PrimLangPos))
			{
				// Map the primary language to the shortest possible one
				OutTag.PrimaryLanguage = ISO639::MapTo639_1(OutTag.FullLanguage.Mid(0, PrimLangPos));
				OutTag.ExtendedLanguage = OutTag.FullLanguage.Mid(PrimLangPos + 1);
				// Then reassemble the full language again.
				OutTag.FullLanguage = FString::Printf(TEXT("%s-%s"), *OutTag.PrimaryLanguage, *OutTag.ExtendedLanguage);
			}
			else
			{
				OutTag.FullLanguage = ISO639::MapTo639_1(OutTag.FullLanguage);
				OutTag.PrimaryLanguage = OutTag.FullLanguage;
			}
			// Try script
			FRegexMatcher Script(Regexes::Script(), Remainder);
			if (Script.FindNext())
			{
				OutTag.Script = Script.GetCaptureGroup(1);
				OutTag.Script[0] = FChar::ToUpper(OutTag.Script[0]);
				Remainder.MidInline(1 + OutTag.Script.Len());
				ParsePos += 1 + OutTag.Script.Len();
				LastSuccessfulSubPart = ESubPart::Script;
			}
			// Try region
			FRegexMatcher Region(Regexes::Region(), Remainder);
			if (Region.FindNext())
			{
				OutTag.Region = Region.GetCaptureGroup(1).ToUpper();
				Remainder.MidInline(1 + OutTag.Region.Len());
				ParsePos += 1 + OutTag.Region.Len();
				LastSuccessfulSubPart = ESubPart::Region;
			}
			// Now see if there are any variants
			while(1)
			{
				FRegexMatcher Variant(Regexes::Variant(), Remainder);
				if (Variant.FindNext())
				{
					FString V(Variant.GetCaptureGroup(1));
					if (OutTag.Variants.ContainsByPredicate([&v=V](const FString& e){ return e.Equals(v, ESearchCase::IgnoreCase); }))
					{
						OutError = FString::Printf(TEXT("Variant %s appears more than once"), *V);
						return false;
					}
					OutTag.Variants.Emplace(V);
					Remainder.MidInline(1 + V.Len());
					ParsePos += 1 + V.Len();
					LastSuccessfulSubPart = ESubPart::Variant;
				}
				else
				{
					break;
				}
			}
			// Extensions?
			while(1)
			{
				FRegexMatcher Extension(Regexes::Extension(), Remainder);
				if (Extension.FindNext())
				{
					OutTag.Extensions.Emplace(Extension.GetCaptureGroup(1));
					Remainder.MidInline(1 + OutTag.Extensions.Last().Len());
					ParsePos += 1 + OutTag.Extensions.Last().Len();
					LastSuccessfulSubPart = ESubPart::Extension;
				}
				else
				{
					break;
				}
			}
			// Private use?
			FRegexMatcher PrivateUse(Regexes::PrivateUse(), Remainder);
			if (PrivateUse.FindNext())
			{
				OutTag.PrivateUse = PrivateUse.GetCaptureGroup(1);
				Remainder.MidInline(1 + OutTag.PrivateUse.Len());
				ParsePos += 1 + OutTag.PrivateUse.Len();
				LastSuccessfulSubPart = ESubPart::PrivateUse;
			}
			// We need to have consumed the entire language tag for parsing to be
			// successful. If there is still something left then the tag is malformed.
			if (!Remainder.IsEmpty())
			{
				OutError = FString::Printf(TEXT("Error after %s sub tag at position %d: \"%s\""), SubPartNames[(int32)LastSuccessfulSubPart], ParsePos+1, *Remainder);
				return false;
			}
			return true;
		}



		bool ParseRFC5646Tag(FLanguageTag& OutTag, const FString& InRFC5646)
		{
			FString ErrorMsg;
			return ParseInternal(OutTag, ErrorMsg, InRFC5646.ToLower());
		}

		TArray<int32> FindExtendedFilteringMatch(const TArray<FLanguageTag>& InTagsToCheck, const FString& InRFC4647Ranges)
		{
			TArray<FString> RangesToTest;
			InRFC4647Ranges.ParseIntoArray(RangesToTest, TEXT(","), true);
			// No test range, no result.
			if (RangesToTest.IsEmpty())
			{
				return TArray<int32>();
			}
			// Loop over the candidates
			TArray<int32> ResultIndices;
			for(int32 nCand=0; nCand<InTagsToCheck.Num(); ++nCand)
			{
				// Step 1: break apart the language tag and the language range on the hyphens.
				TArray<FString> CandidateParts;
				InTagsToCheck[nCand].Get().ParseIntoArray(CandidateParts, TEXT("-"), true);
				// Test each language range in turn.
				bool bFoundMatch = false;
				for(int32 nRange=0; !bFoundMatch && nRange<RangesToTest.Num(); ++nRange)
				{
					TArray<FString> RangeParts;
					RangesToTest[nRange].ParseIntoArray(RangeParts, TEXT("-"), true);
					check(!RangeParts.IsEmpty());	// can't be empty since we culled empty ranges on entry.
					// Step 2: check language part
					bool bFirstIsWildcard = RangeParts[0].Equals(TEXT("*"));
					if (CandidateParts.IsEmpty())
					{
						if (bFirstIsWildcard)
						{
							bFoundMatch = true;
							ResultIndices.AddUnique(nCand);
						}
						continue;
					}
					else if (bFirstIsWildcard || RangeParts[0].Equals(CandidateParts[0], ESearchCase::IgnoreCase))
					{
						int32 candPartIdx = 1;
						int32 rngPartIdx = 1;
						bool bMatches = true;
						// Step 3:
						while(rngPartIdx < RangeParts.Num())
						{
							// 3A:
							if (RangeParts[rngPartIdx].Equals(TEXT("*")))
							{
								++rngPartIdx;
								continue;
							}
							// 3B:
							if (candPartIdx >= CandidateParts.Num())
							{
								bMatches = false;
								break;
							}
							// 3C:
							if (RangeParts[rngPartIdx].Equals(CandidateParts[candPartIdx], ESearchCase::IgnoreCase))
							{
								++rngPartIdx;
								++candPartIdx;
								continue;
							}
							// 3D:
							if (CandidateParts[candPartIdx].Len() == 1)
							{
								bMatches = false;
								break;
							}
							// 3E:
							++candPartIdx;
						}
						if (bMatches)
						{
							bFoundMatch = true;
							ResultIndices.AddUnique(nCand);
						}
					}
				}
			}

			// If filtering produced no results try "lookup"
			if (ResultIndices.IsEmpty())
			{
				int32 BestMatchPos = 0;
				int32 BestMatchIndex = -1;
				// Try each language range in priority order.
				for(auto& testRange : RangesToTest)
				{
					// Cannot use language range containing wildcard
					if (testRange.Contains(TEXT("*")))
					{
						continue;
					}
					// Parse the languate range as a tag. If this fails we ignore it.
					FLanguageTag testTag;
					FString parseError;
					if (!ParseInternal(testTag, parseError, testRange))
					{
						continue;
					}

					// Check the parsed language range against the list of given language tags.
					for(int32 nCand=0; nCand<InTagsToCheck.Num(); ++nCand)
					{
						// Compare the sub tags element-wise.
						int32 MatchPos = 0;
						if (testTag.PrimaryLanguage.Equals(InTagsToCheck[nCand].PrimaryLanguage, ESearchCase::IgnoreCase))
						{
							++MatchPos;
							if (testTag.ExtendedLanguage.Equals(InTagsToCheck[nCand].ExtendedLanguage, ESearchCase::IgnoreCase))
							{
								++MatchPos;
								if (testTag.Script.Equals(InTagsToCheck[nCand].Script, ESearchCase::IgnoreCase))
								{
									++MatchPos;
									if (testTag.Region.Equals(InTagsToCheck[nCand].Region, ESearchCase::IgnoreCase))
									{
										++MatchPos;
									}
								}
							}
						}
						// A (better) match?
						if (MatchPos > BestMatchPos)
						{
							BestMatchPos = MatchPos;
							BestMatchIndex = nCand;
						}
					}

					// If the language range matched one of the given language tags we stop.
					// The assumption still is that the language ranges (if more than one) are given
					// in most descriptive to least descriptive order.
					if (BestMatchIndex >= 0)
					{
						ResultIndices.Emplace(BestMatchIndex);
						break;
					}
				}
			}

			return ResultIndices;
		}

	}
}
