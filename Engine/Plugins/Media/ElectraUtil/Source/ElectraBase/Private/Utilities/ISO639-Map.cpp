// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/ISO639-Map.h"

namespace Electra
{
	namespace ISO639
	{
		struct FCodeEntry
		{
			const TCHAR* _2B;		// ISO-639-2B
			const TCHAR* _3;		// ISO-639-3 (same as ISO 639-2T)
			const TCHAR* _1;		// ISO-639-1
		};

		/*
			This map was generated as given by https://en.wikipedia.org/wiki/List_of_ISO_639-2_codes

			Elements for which no 2 letter ISO-639-1 exists have been removed.
		*/

		static TArray<FCodeEntry> CodeMap =
		{
			{TEXT("aar"), TEXT("aar"), TEXT("aa")},	// Afar
			{TEXT("abk"), TEXT("abk"), TEXT("ab")},	// Abkhazian
			{TEXT("afr"), TEXT("afr"), TEXT("af")},	// Afrikaans
			{TEXT("aka"), TEXT("aka"), TEXT("ak")},	// Akan
			{TEXT("amh"), TEXT("amh"), TEXT("am")},	// Amharic
			{TEXT("ara"), TEXT("ara"), TEXT("ar")},	// Arabic
			{TEXT("arg"), TEXT("arg"), TEXT("an")},	// Aragonese
			{TEXT("asm"), TEXT("asm"), TEXT("as")},	// Assamese
			{TEXT("ava"), TEXT("ava"), TEXT("av")},	// Avaric
			{TEXT("ave"), TEXT("ave"), TEXT("ae")},	// Avestan
			{TEXT("aym"), TEXT("aym"), TEXT("ay")},	// Aymara
			{TEXT("aze"), TEXT("aze"), TEXT("az")},	// Azerbaijani
			{TEXT("bak"), TEXT("bak"), TEXT("ba")},	// Bashkir
			{TEXT("bam"), TEXT("bam"), TEXT("bm")},	// Bambara
			{TEXT("bel"), TEXT("bel"), TEXT("be")},	// Belarusian
			{TEXT("ben"), TEXT("ben"), TEXT("bn")},	// Bengali
			{TEXT("bis"), TEXT("bis"), TEXT("bi")},	// Bislama
			{TEXT("tib"), TEXT("bod"), TEXT("bo")},	// Tibetan
			{TEXT("bos"), TEXT("bos"), TEXT("bs")},	// Bosnian
			{TEXT("bre"), TEXT("bre"), TEXT("br")},	// Breton
			{TEXT("bul"), TEXT("bul"), TEXT("bg")},	// Bulgarian
			{TEXT("cat"), TEXT("cat"), TEXT("ca")},	// Catalan; Valencian
			{TEXT("cze"), TEXT("ces"), TEXT("cs")},	// Czech
			{TEXT("cha"), TEXT("cha"), TEXT("ch")},	// Chamorro
			{TEXT("che"), TEXT("che"), TEXT("ce")},	// Chechen
			{TEXT("chu"), TEXT("chu"), TEXT("cu")},	// Church Slavic; Old Slavonic; Church Slavonic; Old Bulgarian; Old Church Slavonic
			{TEXT("chv"), TEXT("chv"), TEXT("cv")},	// Chuvash
			{TEXT("cor"), TEXT("cor"), TEXT("kw")},	// Cornish
			{TEXT("cos"), TEXT("cos"), TEXT("co")},	// Corsican
			{TEXT("cre"), TEXT("cre"), TEXT("cr")},	// Cree
			{TEXT("wel"), TEXT("cym"), TEXT("cy")},	// Welsh
			{TEXT("dan"), TEXT("dan"), TEXT("da")},	// Danish
			{TEXT("ger"), TEXT("deu"), TEXT("de")},	// German
			{TEXT("div"), TEXT("div"), TEXT("dv")},	// Divehi; Dhivehi; Maldivian
			{TEXT("dzo"), TEXT("dzo"), TEXT("dz")},	// Dzongkha
			{TEXT("gre"), TEXT("ell"), TEXT("el")},	// Greek, Modern (1453–)
			{TEXT("eng"), TEXT("eng"), TEXT("en")},	// English
			{TEXT("epo"), TEXT("epo"), TEXT("eo")},	// Esperanto
			{TEXT("est"), TEXT("est"), TEXT("et")},	// Estonian
			{TEXT("baq"), TEXT("eus"), TEXT("eu")},	// Basque
			{TEXT("ewe"), TEXT("ewe"), TEXT("ee")},	// Ewe
			{TEXT("fao"), TEXT("fao"), TEXT("fo")},	// Faroese
			{TEXT("per"), TEXT("fas"), TEXT("fa")},	// Persian
			{TEXT("fij"), TEXT("fij"), TEXT("fj")},	// Fijian
			{TEXT("fin"), TEXT("fin"), TEXT("fi")},	// Finnish
			{TEXT("fre"), TEXT("fra"), TEXT("fr")},	// French
			{TEXT("fry"), TEXT("fry"), TEXT("fy")},	// Western Frisian
			{TEXT("ful"), TEXT("ful"), TEXT("ff")},	// Fulah
			{TEXT("gla"), TEXT("gla"), TEXT("gd")},	// Gaelic; Scottish Gaelic
			{TEXT("gle"), TEXT("gle"), TEXT("ga")},	// Irish
			{TEXT("glg"), TEXT("glg"), TEXT("gl")},	// Galician
			{TEXT("glv"), TEXT("glv"), TEXT("gv")},	// Manx
			{TEXT("grn"), TEXT("grn"), TEXT("gn")},	// Guarani
			{TEXT("guj"), TEXT("guj"), TEXT("gu")},	// Gujarati
			{TEXT("hat"), TEXT("hat"), TEXT("ht")},	// Haitian; Haitian Creole
			{TEXT("hau"), TEXT("hau"), TEXT("ha")},	// Hausa
			{TEXT("heb"), TEXT("heb"), TEXT("he")},	// Hebrew
			{TEXT("her"), TEXT("her"), TEXT("hz")},	// Herero
			{TEXT("hin"), TEXT("hin"), TEXT("hi")},	// Hindi
			{TEXT("hmo"), TEXT("hmo"), TEXT("ho")},	// Hiri Motu
			{TEXT("hrv"), TEXT("hrv"), TEXT("hr")},	// Croatian
			{TEXT("hun"), TEXT("hun"), TEXT("hu")},	// Hungarian
			{TEXT("arm"), TEXT("hye"), TEXT("hy")},	// Armenian
			{TEXT("ibo"), TEXT("ibo"), TEXT("ig")},	// Igbo
			{TEXT("ido"), TEXT("ido"), TEXT("io")},	// Ido
			{TEXT("iii"), TEXT("iii"), TEXT("ii")},	// Sichuan Yi; Nuosu
			{TEXT("iku"), TEXT("iku"), TEXT("iu")},	// Inuktitut
			{TEXT("ile"), TEXT("ile"), TEXT("ie")},	// Interlingue; Occidental
			{TEXT("ina"), TEXT("ina"), TEXT("ia")},	// Interlingua (International Auxiliary Language Association)
			{TEXT("ind"), TEXT("ind"), TEXT("id")},	// Indonesian
			{TEXT("ipk"), TEXT("ipk"), TEXT("ik")},	// Inupiaq
			{TEXT("ice"), TEXT("isl"), TEXT("is")},	// Icelandic
			{TEXT("ita"), TEXT("ita"), TEXT("it")},	// Italian
			{TEXT("jav"), TEXT("jav"), TEXT("jv")},	// Javanese
			{TEXT("jpn"), TEXT("jpn"), TEXT("ja")},	// Japanese
			{TEXT("kal"), TEXT("kal"), TEXT("kl")},	// Kalaallisut; Greenlandic
			{TEXT("kan"), TEXT("kan"), TEXT("kn")},	// Kannada
			{TEXT("kas"), TEXT("kas"), TEXT("ks")},	// Kashmiri
			{TEXT("geo"), TEXT("kat"), TEXT("ka")},	// Georgian
			{TEXT("kau"), TEXT("kau"), TEXT("kr")},	// Kanuri
			{TEXT("kaz"), TEXT("kaz"), TEXT("kk")},	// Kazakh
			{TEXT("khm"), TEXT("khm"), TEXT("km")},	// Central Khmer
			{TEXT("kik"), TEXT("kik"), TEXT("ki")},	// Kikuyu; Gikuyu
			{TEXT("kin"), TEXT("kin"), TEXT("rw")},	// Kinyarwanda
			{TEXT("kir"), TEXT("kir"), TEXT("ky")},	// Kirghiz; Kyrgyz
			{TEXT("kom"), TEXT("kom"), TEXT("kv")},	// Komi
			{TEXT("kon"), TEXT("kon"), TEXT("kg")},	// Kongo
			{TEXT("kor"), TEXT("kor"), TEXT("ko")},	// Korean
			{TEXT("kua"), TEXT("kua"), TEXT("kj")},	// Kuanyama; Kwanyama
			{TEXT("kur"), TEXT("kur"), TEXT("ku")},	// Kurdish
			{TEXT("lao"), TEXT("lao"), TEXT("lo")},	// Lao
			{TEXT("lat"), TEXT("lat"), TEXT("la")},	// Latin
			{TEXT("lav"), TEXT("lav"), TEXT("lv")},	// Latvian
			{TEXT("lim"), TEXT("lim"), TEXT("li")},	// Limburgan; Limburger; Limburgish
			{TEXT("lin"), TEXT("lin"), TEXT("ln")},	// Lingala
			{TEXT("lit"), TEXT("lit"), TEXT("lt")},	// Lithuanian
			{TEXT("ltz"), TEXT("ltz"), TEXT("lb")},	// Luxembourgish; Letzeburgesch
			{TEXT("lub"), TEXT("lub"), TEXT("lu")},	// Luba-Katanga
			{TEXT("lug"), TEXT("lug"), TEXT("lg")},	// Ganda
			{TEXT("mah"), TEXT("mah"), TEXT("mh")},	// Marshallese
			{TEXT("mal"), TEXT("mal"), TEXT("ml")},	// Malayalam
			{TEXT("mar"), TEXT("mar"), TEXT("mr")},	// Marathi
			{TEXT("mac"), TEXT("mkd"), TEXT("mk")},	// Macedonian
			{TEXT("mlg"), TEXT("mlg"), TEXT("mg")},	// Malagasy
			{TEXT("mlt"), TEXT("mlt"), TEXT("mt")},	// Maltese
			{TEXT("mon"), TEXT("mon"), TEXT("mn")},	// Mongolian
			{TEXT("mao"), TEXT("mri"), TEXT("mi")},	// Māori
			{TEXT("may"), TEXT("msa"), TEXT("ms")},	// Malay
			{TEXT("bur"), TEXT("mya"), TEXT("my")},	// Burmese
			{TEXT("nau"), TEXT("nau"), TEXT("na")},	// Nauru
			{TEXT("nav"), TEXT("nav"), TEXT("nv")},	// Navajo; Navaho
			{TEXT("nbl"), TEXT("nbl"), TEXT("nr")},	// Ndebele, South; South Ndebele
			{TEXT("nde"), TEXT("nde"), TEXT("nd")},	// Ndebele, North; North Ndebele
			{TEXT("ndo"), TEXT("ndo"), TEXT("ng")},	// Ndonga
			{TEXT("nep"), TEXT("nep"), TEXT("ne")},	// Nepali
			{TEXT("dut"), TEXT("nld"), TEXT("nl")},	// Dutch; Flemish
			{TEXT("nno"), TEXT("nno"), TEXT("nn")},	// Norwegian Nynorsk; Nynorsk, Norwegian
			{TEXT("nob"), TEXT("nob"), TEXT("nb")},	// Bokmål, Norwegian; Norwegian Bokmål
			{TEXT("nor"), TEXT("nor"), TEXT("no")},	// Norwegian
			{TEXT("nya"), TEXT("nya"), TEXT("ny")},	// Chichewa; Chewa; Nyanja
			{TEXT("oci"), TEXT("oci"), TEXT("oc")},	// Occitan (post 1500)
			{TEXT("oji"), TEXT("oji"), TEXT("oj")},	// Ojibwa
			{TEXT("ori"), TEXT("ori"), TEXT("or")},	// Oriya
			{TEXT("orm"), TEXT("orm"), TEXT("om")},	// Oromo
			{TEXT("oss"), TEXT("oss"), TEXT("os")},	// Ossetian; Ossetic
			{TEXT("pan"), TEXT("pan"), TEXT("pa")},	// Panjabi; Punjabi
			{TEXT("pli"), TEXT("pli"), TEXT("pi")},	// Pali
			{TEXT("pol"), TEXT("pol"), TEXT("pl")},	// Polish
			{TEXT("por"), TEXT("por"), TEXT("pt")},	// Portuguese
			{TEXT("pus"), TEXT("pus"), TEXT("ps")},	// Pushto; Pashto
			{TEXT("que"), TEXT("que"), TEXT("qu")},	// Quechua
			{TEXT("roh"), TEXT("roh"), TEXT("rm")},	// Romansh
			{TEXT("rum"), TEXT("ron"), TEXT("ro")},	// Romanian; Moldavian; Moldovan
			{TEXT("run"), TEXT("run"), TEXT("rn")},	// Rundi
			{TEXT("rus"), TEXT("rus"), TEXT("ru")},	// Russian
			{TEXT("sag"), TEXT("sag"), TEXT("sg")},	// Sango
			{TEXT("san"), TEXT("san"), TEXT("sa")},	// Sanskrit
			{TEXT("sin"), TEXT("sin"), TEXT("si")},	// Sinhala; Sinhalese
			{TEXT("slo"), TEXT("slk"), TEXT("sk")},	// Slovak
			{TEXT("slv"), TEXT("slv"), TEXT("sl")},	// Slovenian
			{TEXT("sme"), TEXT("sme"), TEXT("se")},	// Northern Sami
			{TEXT("smo"), TEXT("smo"), TEXT("sm")},	// Samoan
			{TEXT("sna"), TEXT("sna"), TEXT("sn")},	// Shona
			{TEXT("snd"), TEXT("snd"), TEXT("sd")},	// Sindhi
			{TEXT("som"), TEXT("som"), TEXT("so")},	// Somali
			{TEXT("sot"), TEXT("sot"), TEXT("st")},	// Sotho, Southern
			{TEXT("spa"), TEXT("spa"), TEXT("es")},	// Spanish; Castilian
			{TEXT("alb"), TEXT("sqi"), TEXT("sq")},	// Albanian
			{TEXT("srd"), TEXT("srd"), TEXT("sc")},	// Sardinian
			{TEXT("srp"), TEXT("srp"), TEXT("sr")},	// Serbian
			{TEXT("ssw"), TEXT("ssw"), TEXT("ss")},	// Swati
			{TEXT("sun"), TEXT("sun"), TEXT("su")},	// Sundanese
			{TEXT("swa"), TEXT("swa"), TEXT("sw")},	// Swahili
			{TEXT("swe"), TEXT("swe"), TEXT("sv")},	// Swedish
			{TEXT("tah"), TEXT("tah"), TEXT("ty")},	// Tahitian
			{TEXT("tam"), TEXT("tam"), TEXT("ta")},	// Tamil
			{TEXT("tat"), TEXT("tat"), TEXT("tt")},	// Tatar
			{TEXT("tel"), TEXT("tel"), TEXT("te")},	// Telugu
			{TEXT("tgk"), TEXT("tgk"), TEXT("tg")},	// Tajik
			{TEXT("tgl"), TEXT("tgl"), TEXT("tl")},	// Tagalog
			{TEXT("tha"), TEXT("tha"), TEXT("th")},	// Thai
			{TEXT("tir"), TEXT("tir"), TEXT("ti")},	// Tigrinya
			{TEXT("ton"), TEXT("ton"), TEXT("to")},	// Tonga (Tonga Islands)
			{TEXT("tsn"), TEXT("tsn"), TEXT("tn")},	// Tswana
			{TEXT("tso"), TEXT("tso"), TEXT("ts")},	// Tsonga
			{TEXT("tuk"), TEXT("tuk"), TEXT("tk")},	// Turkmen
			{TEXT("tur"), TEXT("tur"), TEXT("tr")},	// Turkish
			{TEXT("twi"), TEXT("twi"), TEXT("tw")},	// Twi
			{TEXT("uig"), TEXT("uig"), TEXT("ug")},	// Uighur; Uyghur
			{TEXT("ukr"), TEXT("ukr"), TEXT("uk")},	// Ukrainian
			{TEXT("urd"), TEXT("urd"), TEXT("ur")},	// Urdu
			{TEXT("uzb"), TEXT("uzb"), TEXT("uz")},	// Uzbek
			{TEXT("ven"), TEXT("ven"), TEXT("ve")},	// Venda
			{TEXT("vie"), TEXT("vie"), TEXT("vi")},	// Vietnamese
			{TEXT("vol"), TEXT("vol"), TEXT("vo")},	// Volapük
			{TEXT("wln"), TEXT("wln"), TEXT("wa")},	// Walloon
			{TEXT("wol"), TEXT("wol"), TEXT("wo")},	// Wolof
			{TEXT("xho"), TEXT("xho"), TEXT("xh")},	// Xhosa
			{TEXT("yid"), TEXT("yid"), TEXT("yi")},	// Yiddish
			{TEXT("yor"), TEXT("yor"), TEXT("yo")},	// Yoruba
			{TEXT("zha"), TEXT("zha"), TEXT("za")},	// Zhuang; Chuang
			{TEXT("chi"), TEXT("zho"), TEXT("zh")},	// Chinese
			{TEXT("zul"), TEXT("zul"), TEXT("zu")}	// Zulu
		};

		// Undefined
		static const TCHAR* const CodeUnd(TEXT("und"));

		const TCHAR* Get639_1(const FString& InFrom639_1_2_3)
		{
			// Already a 2 letter code?
			if (InFrom639_1_2_3.Len() == 2)
			{
				// Look it up in the table if it exists.
				int32 Index =  CodeMap.IndexOfByPredicate([&](const FCodeEntry& Entry) { return InFrom639_1_2_3.Equals(Entry._1); });
				if (Index != INDEX_NONE)
				{
					return CodeMap[Index]._1;
				}
			}
			else if (InFrom639_1_2_3.Len() == 3)
			{
				// Three letter code, either ISO-639-3 or ISO-639-2B. Look it up in either table.
				int32 Index =  CodeMap.IndexOfByPredicate([&](const FCodeEntry& Entry) { return InFrom639_1_2_3.Equals(Entry._3); });
				if (Index == INDEX_NONE)
				{
					Index =  CodeMap.IndexOfByPredicate([&](const FCodeEntry& Entry) { return InFrom639_1_2_3.Equals(Entry._2B); });
				}
				if (Index != INDEX_NONE)
				{
					return CodeMap[Index]._1;
				}
				else
				{
					/*
						The standard includes some codes for special situations:
							mis, for "uncoded languages"
							mul, for "multiple languages"
							qaa-qtz, a range reserved for local use
							und, for "undetermined"
							zxx, for "no linguistic content; not applicable"
					*/
					if (InFrom639_1_2_3.Equals(TEXT("mis")) || InFrom639_1_2_3.Equals(TEXT("zxx")) || InFrom639_1_2_3.Equals(TEXT("mul")))
					{
						// Map those to "und", even though this is not a 639-1 code!
						return CodeUnd;
					}
				}
			}
			return nullptr;
		}


		FString MapTo639_1(const FString& InFrom639_1_2_3)
		{
			const TCHAR* Mapped = Get639_1(InFrom639_1_2_3);
			return Mapped ? Mapped : InFrom639_1_2_3;
		}


		FString RFC5646To639_1(const FString& InFromRFC5646)
		{
			FString s(InFromRFC5646);
			// We only look at the primary language tag before the first '-',
			// which by definition happens to be the "shortest ISO 639 code".
			int32 DashPos = INDEX_NONE;
			if (s.FindChar(TCHAR('-'), DashPos))
			{
				s.LeftInline(DashPos);
			}
			return MapTo639_1(s);
		}

	}
}
