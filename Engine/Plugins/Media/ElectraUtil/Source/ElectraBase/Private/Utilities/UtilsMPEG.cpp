// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/UtilsMPEG.h"
#include "Utilities/Utilities.h"
#include "Utilities/ElectraBitstream.h"

namespace Electra
{
	namespace MPEG
	{
		class FMP4BitReader : public FBitstreamReader
		{
		public:
			FMP4BitReader(const void* pData, int64 nData)
				: FBitstreamReader(pData, nData)
			{
			}
			int32 ReadMP4Length()
			{
				int32 Length = 0;
				for (int32 i = 0; i < 4; ++i)
				{
					uint32 Bits = GetBits(8);
					Length = (Length << 7) + (Bits & 0x7f);
					if ((Bits & 0x80) == 0)
						break;
				}
				return Length;
			}
		};


		FESDescriptor::FESDescriptor()
			: ObjectTypeID(FObjectTypeID::Unknown)
			, StreamTypeID(FStreamType::Unknown)
			, BufferSize(0)
			, MaxBitrate(0)
			, AvgBitrate(0)
			, ESID(0)
			, DependsOnStreamESID(0)
			, StreamPriority(16)
			, bDependsOnStream(false)
		{
		}

		void FESDescriptor::SetRawData(const void* Data, int64 Size)
		{
			RawData.Empty();
			if (Size)
			{
				RawData.Reserve((uint32)Size);
				RawData.SetNumUninitialized((uint32)Size);
				FMemory::Memcpy(RawData.GetData(), Data, Size);
			}
		}

		const TArray<uint8>& FESDescriptor::GetRawData() const
		{
			return RawData;
		}

		const TArray<uint8>& FESDescriptor::GetCodecSpecificData() const
		{
			return CodecSpecificData;
		}


		bool FESDescriptor::Parse()
		{
			CodecSpecificData.Empty();

			FMP4BitReader BitReader(RawData.GetData(), RawData.Num());

			if (BitReader.GetBits(8) != 3)
			{
				return false;
			}

			int32 ESSize = BitReader.ReadMP4Length();

			ESID = (uint16)BitReader.GetBits(16);
			bDependsOnStream = BitReader.GetBits(1) != 0;
			bool bURLFlag = BitReader.GetBits(1) != 0;
			bool bOCRflag = BitReader.GetBits(1) != 0;
			StreamPriority = (uint8)BitReader.GetBits(5);
			if (bDependsOnStream)
			{
				DependsOnStreamESID = BitReader.GetBits(16);
			}
			if (bURLFlag)
			{
				// Skip over the URL
				uint32 urlLen = BitReader.GetBits(8);
				BitReader.SkipBytes(urlLen);
			}
			if (bOCRflag)
			{
				// Skip the OCR ES ID
				BitReader.SkipBits(16);
			}

			// Parse the config descriptor
			if (BitReader.GetBits(8) != 4)
			{
				return false;
			}
			int32 ConfigDescrSize = BitReader.ReadMP4Length();
			ObjectTypeID = static_cast<FObjectTypeID>(BitReader.GetBits(8));
			StreamTypeID = static_cast<FStreamType>(BitReader.GetBits(6));
			// Skip upstream flag
			BitReader.SkipBits(1);
			BitReader.SkipBits(1);	// this bit is reserved and must be 1, but it is sometimes incorrectly set to 0, so do not check for it being 1!
			BufferSize = BitReader.GetBits(24);
			MaxBitrate = BitReader.GetBits(32);
			AvgBitrate = BitReader.GetBits(32);
			if (ConfigDescrSize > 13)
			{
				// Optional codec specific descriptor
				if (BitReader.GetBits(8) != 5)
				{
					return false;
				}
				int32 CodecSize = BitReader.ReadMP4Length();
				CodecSpecificData.Reserve(CodecSize);
				for (int32 i = 0; i < CodecSize; ++i)
				{
					CodecSpecificData.Push(BitReader.GetBits(8));
				}
			}

			// SL config (we do not need it, we require it to be there though as per the standard)
			if (BitReader.GetBits(8) != 6)
			{
				return false;
			}
			int32 nSLSize = BitReader.ReadMP4Length();
			if (nSLSize != 1)
			{
				return false;
			}
			if (BitReader.GetBits(8) != 2)
			{
				return false;
			}

			return true;
		}





		bool FID3V2Metadata::Parse(const uint8* InData, int64 InDataSize)
		{
			if (!(InData[0] == 'I' && InData[1] == 'D' && InData[2] == '3' &&
				InData[3] != 0xff && InData[4] != 0xff && InData[6] < 0x80 && InData[7] < 0x80 && InData[8] < 0x80 && InData[9] < 0x80))
			{
				return false;
			}
			auto GetID3Size = [](const uint8* InAddr) -> int32
			{
				return (int32) (10U + (static_cast<uint32>(InAddr[0]) << 21) + (static_cast<uint32>(InAddr[1]) << 14) + (static_cast<uint32>(InAddr[2]) << 7) + InAddr[3]);
			};
			auto GetTagSize = [](const uint8* InAddr) -> int32
			{
				return (int32) (10U + (static_cast<uint32>(InAddr[0]) << 24) + (static_cast<uint32>(InAddr[1]) << 16) + (static_cast<uint32>(InAddr[2]) << 8) + InAddr[3]);
			};
			if (GetID3Size(InData + 6) > InDataSize)
			{
				return false;
			}
			// "Unsynchronization" is not currently supported.
			if ((InData[5] & 0x80) != 0)
			{
				check(!"unsynchronization not currently supported");
				return false;
			}
			// Experimental headers are not supported
			if ((InData[5] & 0x20) != 0)
			{
				return false;
			}
			int32 HeaderSize = 10;
			if ((InData[5] & 0x40) != 0)
			{
				check(!"extended header not currently supported");
				return false;
				//HeaderSize += GetTagSize(InData + 10) + xxxx;
			}
			const uint8* InDataEnd = InData + InDataSize;
			InData += HeaderSize;
			auto GetString = [](const uint8* InStr, int32 NumBytes, const uint8** OutPtr=nullptr, int32* InOutEncoding=nullptr) -> FString
			{
				if (InStr && NumBytes > 0)
				{
					int32 Encoding = (!InOutEncoding || (InOutEncoding && *InOutEncoding<0)) ? *InStr++ : *InOutEncoding;
					if (InOutEncoding)
					{
						*InOutEncoding = Encoding;
					}
					if (Encoding == 0)
					{
						// ISO 8859-1 encoding
						TArray<uint8> ConvBuf;
						ConvBuf.Reserve(NumBytes*2);
						const uint8* SrcPtr = reinterpret_cast<const uint8*>(InStr);
						int32 NumSrcChars = NumBytes - 1;
						for(int32 i=0; i<NumSrcChars; ++i, ++SrcPtr)
						{
							if (*SrcPtr == 0x00)
							{
								++SrcPtr;
								break;
							}
							// Permitted whitespaces
							if (*SrcPtr == 0x0a || *SrcPtr == 0x09)
							{
								ConvBuf.Add(*SrcPtr);
							}
							else if (*SrcPtr >= 0x20 && *SrcPtr < 0x80)
							{
								ConvBuf.Add(*SrcPtr);
							}
							else  if (*SrcPtr >= 0xa0)
							{
								// We can convert straight from ISO 8859-1 to UTF8 by doing this:
								ConvBuf.Add(0xc0 | (*SrcPtr >> 6));
								ConvBuf.Add(0x80 | (*SrcPtr & 0x3f));
							}
						}
						if (OutPtr)
						{
							*OutPtr = SrcPtr;
						}
						auto Cnv = StringCast<TCHAR>(reinterpret_cast<const UTF8CHAR*>(ConvBuf.GetData()), ConvBuf.Num());
						return FString::ConstructFromPtrSize(Cnv.Get(), Cnv.Length());
					}
					else if (Encoding == 1 && NumBytes > 1)
					{
						const uint8* SrcPtr = reinterpret_cast<const uint8*>(InStr);
						if (OutPtr)
						{
							*OutPtr = SrcPtr + NumBytes;
						}
						int32 NumSrcChars = NumBytes - 1;
						uint16 BOM = ((uint16)SrcPtr[0] << 8) | SrcPtr[1];
						// Have BOM?
						if (BOM == 0xfffe)
						{
							auto Cnv = StringCast<TCHAR>(reinterpret_cast<const UCS2CHAR*>(SrcPtr+2), NumSrcChars/2-1);
							return FString::ConstructFromPtrSize(Cnv.Get(), Cnv.Length());
						}
						else if (BOM == 0xfeff)
						{
							TArray<uint8> be;
							be.AddUninitialized(NumSrcChars);
							for(int32 k=0; k<NumSrcChars; k+=2)
							{
								be[k+0] = SrcPtr[k+1];
								be[k+1] = SrcPtr[k+0];
							}
							auto Cnv = StringCast<TCHAR>(reinterpret_cast<const UCS2CHAR*>(be.GetData()+2), NumSrcChars/2-1);
							return FString::ConstructFromPtrSize(Cnv.Get(), Cnv.Length());
						}
					}
				}
				return FString();
			};
			while(InData+10 < InDataEnd)
			{
				if (((InData[0] >= 'A' && InData[0] <= 'Z') || (InData[0] >= '0' && InData[0] <= '9')) &&
					((InData[1] >= 'A' && InData[1] <= 'Z') || (InData[1] >= '0' && InData[1] <= '9')) &&
					((InData[2] >= 'A' && InData[2] <= 'Z') || (InData[2] >= '0' && InData[2] <= '9')) &&
					((InData[3] >= 'A' && InData[3] <= 'Z') || (InData[3] >= '0' && InData[3] <= '9')))
				{
					uint32 FrameID = (static_cast<uint32>(InData[0]) << 24) | (static_cast<uint32>(InData[1]) << 16) | (static_cast<uint32>(InData[2]) << 8) | static_cast<uint32>(InData[3]);
					int32 FrameSize = (int32)GetTagSize(InData + 4);
					if (FrameSize <= 10)
					{
						return false;
					}
					const uint8* const FrameEnd = InData + FrameSize;
					uint16 Flags = (InData[8] << 8) | InData[9];
					// Ignore compressed, encrypted or grouped tags.
					if ((Flags & 0xc0) != 0)
					{
						InData += FrameSize;
						continue;
					}
					switch(FrameID)
					{
						default:
						{
							break;
						}
						// Track length
						case Utils::Make4CC('T','L','E','N'):
						{
							FString ms(GetString(InData + 10, FrameSize - 10));
							int64 dur = -1;
							LexFromString(dur, *ms);
							if (dur > 0)
							{
								FTimespan Duration = FTimespan::FromMilliseconds((double)dur);
								FItem Item {.Value = FVariant(Duration) };
								Tags.Emplace(FrameID, MoveTemp(Item));
							}
							break;
						}
						// MPEG location lookup table
						case Utils::Make4CC('M','L','L','T'):
						{
							TArray<uint8> Blob(InData + 10, FrameSize - 10);
							FItem Item {.Value = FVariant(MoveTemp(Blob)) };
							Tags.Emplace(FrameID, MoveTemp(Item));
							break;
						}
						// Cover image
						case Utils::Make4CC('A','P','I','C'):
						{
							const uint8* NextData = InData + 10;
							int32 TextEncoding = -1;
							FString MimeType(GetString(NextData, FrameEnd - NextData, &NextData, &TextEncoding));
							if (NextData < FrameEnd)
							{
								uint8 PictureType = *NextData++;
								FString Description(GetString(NextData, FrameEnd - NextData, &NextData, &TextEncoding));
								int32 ImageSize = FrameEnd - NextData;
								TArray<uint8> PicData(NextData, ImageSize);
								FItem Item {.MimeType = MoveTemp(MimeType), .Value = FVariant(MoveTemp(PicData)), .ItemType = PictureType };
								Tags.Emplace(FrameID, MoveTemp(Item));
							}
							break;
						}
						// Private data
						case Utils::Make4CC('P','R','I','V'):
						{
							int32 TextEncoding = 0;
							const uint8* NextData = InData + 10;
							FString Owner(GetString(NextData, FrameEnd - NextData, &NextData, &TextEncoding));
							int32 PrivateSize = FrameEnd - NextData;
							TArray<uint8> PrivateData(NextData, PrivateSize);
							FItem& Item = PrivateItems.Emplace_GetRef();
							Item.MimeType = MoveTemp(Owner);
							Item.Value = FVariant(MoveTemp(PrivateData));
							break;
						}
						// Recognized text fields
						case Utils::Make4CC('T','A','L','B'):
						case Utils::Make4CC('T','C','O','M'):
						case Utils::Make4CC('T','C','O','N'):
						case Utils::Make4CC('T','C','O','P'):
						case Utils::Make4CC('T','D','A','T'):
						case Utils::Make4CC('T','E','N','C'):
						case Utils::Make4CC('T','E','X','T'):
						case Utils::Make4CC('T','I','M','E'):
						case Utils::Make4CC('T','I','T','1'):
						case Utils::Make4CC('T','I','T','2'):
						case Utils::Make4CC('T','I','T','3'):
						case Utils::Make4CC('T','L','A','N'):
						case Utils::Make4CC('T','P','E','1'):
						case Utils::Make4CC('T','P','E','2'):
						case Utils::Make4CC('T','P','E','3'):
						case Utils::Make4CC('T','P','E','4'):
						case Utils::Make4CC('T','P','O','S'):
						case Utils::Make4CC('T','P','U','B'):
						case Utils::Make4CC('T','R','C','K'):
						case Utils::Make4CC('T','Y','E','R'):
						{
							FString s(GetString(InData + 10, FrameSize - 10));
							if (s.Len())
							{
								FItem Item {.Value = FVariant(MoveTemp(s)) };
								Tags.Emplace(FrameID, MoveTemp(Item));
							}
							break;
						}
					}
					InData += FrameSize;
				}
				else
				{
					return InData[0] == 0;
				}
			}
			return true;
		}
		bool FID3V2Metadata::HaveTag(uint32 InTag)
		{
			const FItem* v = Tags.Find(InTag);
			return !!v;
		}
		bool FID3V2Metadata::GetTag(FItem& OutValue, uint32 InTag)
		{
			const FItem* v = Tags.Find(InTag);
			if (v)
			{
				OutValue = *v;
			}
			return !!v;
		}
		const TMap<uint32, FID3V2Metadata::FItem>& FID3V2Metadata::GetTags() const
		{
			return Tags;
		}

		TMap<uint32, FID3V2Metadata::FItem>& FID3V2Metadata::GetTags()
		{
			return Tags;
		}

		const TArray<FID3V2Metadata::FItem>& FID3V2Metadata::GetPrivateItems() const
		{
			return PrivateItems;
		}

	} // namespace MPEG
} // namespace Electra

