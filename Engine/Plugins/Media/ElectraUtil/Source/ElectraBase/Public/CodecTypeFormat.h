// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Misc/TVariant.h"
#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"

#include "Utilities/BCP47-Helpers.h"

#define UE_API ELECTRABASE_API

namespace Electra
{

	/**
	 * This structure contains codec format related information.
	 * Only information pertaining to the codec that can be derived from
	 * the container, the RFC6381 format string or a media stream playlist
	 * should be stored here.
	 */
	struct FCodecTypeFormat
	{
		enum class EType
		{
			Video,
			Audio,
			Subtitle,
			Timecode,
			Invalid,
			MAX = Invalid
		};
		enum class EKeyframeMode
		{
			Unknown,
			OnlyKeyframes,
			DeltaFrames
		};
		struct FVideo
		{
			// Codec specific profile, level, etc.
			struct FProfile
			{
				uint32 Tier = 0;
				uint32 ProfileSpace = 0;
				uint32 Profile = 0;
				uint32 Level = 0;
				uint64 Constraints = 0;
				uint64 CompatibilityFlags = 0;
			};
			// Codec specific color information. These do not necessarily correspond to MPEG definitions!
			struct FColorInfo
			{
				uint8 colourPrimaries = 0;
				uint8 transferCharacteristics = 0;
				uint8 matrixCoefficients = 0;
				uint8 videoFullRangeFlag = 0;
				uint8 chromaSubsampling = 0;
				struct FChromaInfo
				{
					uint8 monochrome = 0;
					uint8 chromaSubsamplingX = 0;
					uint8 chromaSubsamplingY = 0;
					uint8 chromaSamplingPosition = 0;
				};
				TOptional<FChromaInfo> ChromaInfo;
			};
			FFrameRate FrameRate {0, 0};
			uint32 Width = 0;
			uint32 Height = 0;
			uint32 BitDepth = 0;
			uint32 AspectRatioW = 1;
			uint32 AspectRatioH = 1;
			FProfile Profile;
			TOptional<FColorInfo> OptColorInfo;
		};
		struct FAudio
		{
			struct FMPEGObjectType
			{
				uint32 ObjectType = 0;
				uint32 AudioObjectType = 0;
				uint8 MPEG = 0;
				uint8 Layer = 0;
			};

			uint32 NumChannels = 0;
			uint32 ChannelConfiguration = 0;
			uint32 SampleRate = 0;
			TVariant<FEmptyVariantState, FMPEGObjectType> ObjectType;
		};
		struct FSubtitle
		{
		};
		struct FTMCDTimecode
		{
			// See: https://developer.apple.com/documentation/quicktime-file-format/timecode_sample_description/flags
			enum EFlags : uint32
			{
				DropFrame = 0x0001,					// Indicates whether the timecode is drop frame. Set it to 1 if the timecode is drop frame.
				Max24Hour = 0x0002,					// Indicates whether the timecode wraps after 24 hours. Set it to 1 if the timecode wraps.
				AllowNegativeTimes = 0x0004,		// Indicates whether negative time values are allowed. Set it to 1 if the timecode supports negative values.
				Counter = 0x0008					// Indicates whether the time value corresponds to a tape counter value. Set it to 1 if the timecode values are tape counter values.
			};

			uint32 Flags = 0;
			uint32 Timescale = 0;
			uint32 FrameDuration = 0;
			uint32 NumberOfFrames = 0;

			bool IsDropFrame() const
			{ return !!(Flags & EFlags::DropFrame); }

			bool WrapsAfter24Hours() const
			{ return !!(Flags & EFlags::Max24Hour); }

			bool SupportsNegativeTime() const
			{ return !!(Flags & EFlags::AllowNegativeTimes); }

			FFrameRate GetFrameRate() const
			{ return IsDropFrame() ? FFrameRate(Timescale, FrameDuration) : FFrameRate(NumberOfFrames, 1); }

			FTimecode ConvertToTimecode(uint32 InSampleTimecode) const
			{
				// Needs to be an int32 for use with the following methods.
				check(InSampleTimecode <= 0x7fffffffU);
				const FFrameRate FrameRate = GetFrameRate();
				// Convert to time code (apply roll over, etc). via conversion to seconds first.
				return FTimecode(FrameRate.AsSeconds(FFrameTime(FFrameNumber(static_cast<int32>(InSampleTimecode)))), FrameRate, IsDropFrame(), WrapsAfter24Hours());
			}

		};
		TVariant<FEmptyVariantState,FVideo,FAudio,FSubtitle,FTMCDTimecode> Properties;
		BCP47::FLanguageTag LanguageTag;
		FString HumanReadableFormatInfo;
		FString RFC6381;
		FString MimeType;
		TMap<uint32, TArray<uint8>> ExtraBoxes;
		TArray<uint8> DCR;
		TArray<uint8> CSD;
		EType Type = EType::Invalid;
		EKeyframeMode KeyframeMode = EKeyframeMode::Unknown;
		uint32 FourCC = 0;
		uint32 Bitrate = 0;
		uint32 AverageBitrate = 0;
	};


	/**
	 * Encryption information of a track.
	 */
	struct FDRMTypeFormat
	{
		struct FNullEncryptionInfo
		{
		};

		// As defined by ISO/IEC 14496-12 Section 8.12 and ISO/IEC 23001-7
		struct FISOEncryptionInfo
		{
			// Set by `schm` box, if it exists.
			uint32 Scheme = 0;
			uint32 SchemeVersion = 0;

			// Set by `tenc` box, if it exists.
			struct FBlockPattern
			{
				uint8 CryptByteBlock = 0;
				uint8 SkipByteBlock = 0;
			};
			TArray<uint8> DefaultKID;
			TArray<uint8> DefaultIV;
			TOptional<FBlockPattern> BlockPattern;
			int32 DefaultIVSize = 0;

			// Set by `pssh` boxes, if they exist.
			struct FCDMInfo
			{
				TArray<uint8> SystemID;
				TArray<TArray<uint8>> KIDs;
				TArray<uint8> Data;
			};
			TArray<FCDMInfo> CDMInfos;
		};

		TVariant<FNullEncryptionInfo, FISOEncryptionInfo> EncryptionInfo;
		bool IsEncrypted() const
		{ return !EncryptionInfo.IsType<FNullEncryptionInfo>(); }
	};

	struct FDecoderInformation
	{
		bool bIsDecodable = true;
		struct FProviderInfo
		{
			FString Name;
			FString Version;
			FString Implementation;
			FString Vendor;
		};
		FProviderInfo ProviderInfo;
	};

} // namespace Electra

#undef UE_API
