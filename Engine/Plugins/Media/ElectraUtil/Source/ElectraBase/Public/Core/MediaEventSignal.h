// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "Core/MediaNoncopyable.h"


class FMediaEvent : private TMediaNoncopyable<FMediaEvent>
{
public:
	FMediaEvent() : Event(FPlatformProcess::GetSynchEventFromPool(true))
	{ }

	~FMediaEvent()
	{
		Event->Trigger();
		FPlatformProcess::ReturnSynchEventToPool(Event);
	}

	/**
	 * Waits for the event signal to get set. If already set, this function returns immediately.
	 */
	void Wait()
	{
		Event->Wait();
	}

	/**
	 * Waits for the event signal to get set and clears it immediately. If already set, this function returns immediately.
	 */
	void WaitAndReset()
	{
		Event->Wait();
		Event->Reset();
	}

	/**
	 * Signals the event to be set. If already set, this does nothing.
	 */
	void Signal()
	{
		Event->Trigger();
	}

	/**
	 * Clears the signal. Threads will now wait again until the signal is set once more.
	 */
	void Reset()
	{
		Event->Reset();
	}

	/**
	 * Returns state of the signal.
	 */
	bool IsSignaled() const
	{
		return Event->Wait(0);
	}

	/**
	 * Waits for the event signal to get set within the given time limit.
	 * If already set, this function returns immediately. Returns true when event was set, false when wait timed out.
	 */
	bool WaitTimeout(int64 MicroSeconds)
	{
		return Event->Wait(FTimespan::FromMicroseconds(MicroSeconds));
	}

	/**
	 * Waits for the event signal to get set within the given time limit and clears it immediately.
	 * If already set, this function returns immediately. Returns true when event was set, false when wait timed out.
	 */
	bool WaitTimeoutAndReset(int64 MicroSeconds)
	{
		check(MicroSeconds > 0);	// 0 is forbidden. either call Wait() without timeout, or check the signal using IsSignaled()
		bool bGot = Event->Wait(FTimespan::FromMicroseconds(MicroSeconds));
		if (bGot)
		{
			Event->Reset();
		}
		return bGot;
	}

private:
	mutable FEvent* Event;
};
