// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/MediaTypes.h"
#include "Core/MediaMacros.h"
#include "CoreGlobals.h"
#include "HAL/PlatformAtomics.h"

inline uint32 FMediaInterlockedRead(uint32 volatile& variable)
{
	return (uint32)FPlatformAtomics::AtomicRead_Relaxed((volatile const int32*)&variable);
}

inline uint32 FMediaInterlockedIncrement(uint32 volatile& variable)
{
	// The UE implementation of FPlatformAtomics returnes the new value. But media assumes that the value before the operaton is returned
	return (uint32)(FPlatformAtomics::InterlockedIncrement((volatile int32*)&variable) - 1);
}

inline uint32 FMediaInterlockedDecrement(uint32 volatile& variable)
{
	// The UE implementation of FPlatformAtomics returnes the new value. But media assumes that the value before the operaton is returned
	return (uint32)(FPlatformAtomics::InterlockedDecrement((volatile int32*)&variable) + 1);
}

inline uint32 FMediaInterlockedAdd(uint32 volatile& variable, uint32 value)
{
	return (uint32)FPlatformAtomics::InterlockedAdd((volatile int32*)&variable, (int32)value);
}

inline uint32 FMediaInterlockedExchange(uint32 volatile& variable, uint32 exchangeValue)
{
	return (uint32)FPlatformAtomics::InterlockedExchange((volatile int32*)&variable, (int32)exchangeValue);
}

inline void* FMediaInterlockedExchangePointerVoid(void* volatile& variable, void* pExchangeValue)
{
	return FPlatformAtomics::InterlockedExchangePtr((void**)&variable, pExchangeValue);
}

inline uint32 FMediaInterlockedCompareExchange(uint32 volatile& variable, uint32 exchangeValue, uint32 compareValue)
{
	return (uint32)FPlatformAtomics::InterlockedCompareExchange((volatile int32*)&variable, (int32)exchangeValue, (int32)compareValue);
}

inline void* FMediaInterlockedCompareExchangePointer(void* volatile& variable, void* pExchangeValue, void* pCompareValue)
{
	return (void*)FPlatformAtomics::InterlockedCompareExchange((volatile int64*)&variable, (int64)pExchangeValue, (int64)pCompareValue);
}

inline uint64 FMediaInterlockedRead64(uint64 volatile& variable)
{
	return (uint64)FPlatformAtomics::AtomicRead_Relaxed((volatile const int64*)&variable);
}

inline uint64 FMediaInterlockedAdd64(uint64 volatile& variable, uint64 value)
{
	return (uint64)FPlatformAtomics::InterlockedAdd((volatile int64*)&variable, (int64)value);
}

inline int32 FMediaInterlockedRead(int32 volatile& variable)
{
	return FPlatformAtomics::AtomicRead_Relaxed(&variable);
}

inline int32 FMediaInterlockedIncrement(int32 volatile& variable)
{
	// The UE implementation of FPlatformAtomics returnes the new value. But media assumes that the value before the operaton is returned
	return FPlatformAtomics::InterlockedIncrement(&variable) - 1;
}

inline int32 FMediaInterlockedDecrement(int32 volatile& variable)
{
	// The UE implementation of FPlatformAtomics returnes the new value. But media assumes that the value before the operaton is returned
	return FPlatformAtomics::InterlockedDecrement(&variable) + 1;
}

inline int32 FMediaInterlockedAdd(int32 volatile& variable, int32 value)
{
	return FPlatformAtomics::InterlockedAdd(&variable, value);
}

inline int32 FMediaInterlockedExchange(int32 volatile& variable, int32 exchangeValue)
{
	return FPlatformAtomics::InterlockedExchange(&variable, exchangeValue);
}

inline int32 FMediaInterlockedCompareExchange(int32 volatile& variable, int32 exchangeValue, int32 compareValue)
{
	return FPlatformAtomics::InterlockedCompareExchange(&variable, exchangeValue, compareValue);
}

template <typename T, typename X>
inline T* TMediaInterlockedExchangePointer(T* volatile& variable, X* pExchangeValue)
{
	return((T*)FMediaInterlockedExchangePointerVoid((void* volatile&)variable, (void*)pExchangeValue));
}
