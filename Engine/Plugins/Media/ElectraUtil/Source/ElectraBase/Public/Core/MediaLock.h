// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"
#include "HAL/CriticalSection.h"
#include "Core/MediaNoncopyable.h"

/**
 * Empty lock class.
 */
class FMediaLockNone : private TMediaNoncopyable<FMediaLockNone>
{
public:
	inline void Lock() const {}
	void Unlock() const {}

	class ScopedLock : private TMediaNoncopyable<ScopedLock>
	{
	public:
		explicit ScopedLock(const FMediaLockNone& lock) : LockToUse(lock)
		{ }
		~ScopedLock() = default;
	private:
		ScopedLock();
		const FMediaLockNone& LockToUse;
	};
};



/**
 * Lock class for generic container classes requiring mutual exclusion access.
 */
class FMediaLockCriticalSection : private TMediaNoncopyable<FMediaLockCriticalSection>
{
public:
	FMediaLockCriticalSection()
	{
		CriticalSection.Lock();
		CriticalSection.Unlock();
	}

	void Lock() const
	{
		CriticalSection.Lock();
	}
	void Unlock() const
	{
		CriticalSection.Unlock();
	}

	class ScopedLock : private TMediaNoncopyable<ScopedLock>
	{
	public:
		explicit ScopedLock(const FMediaLockCriticalSection& lock) : LockToUse(lock)
		{
			LockToUse.Lock();
		}
		~ScopedLock()
		{
			LockToUse.Unlock();
		}
	private:
		ScopedLock();
		const FMediaLockCriticalSection& LockToUse;
	};
private:
	mutable FCriticalSection CriticalSection;
};
