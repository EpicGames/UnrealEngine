// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/MediaTypes.h"

//--------------------------------------------------------------------------
//
// Array convenience macros
//
// Evaluates to the number of elements in a statically defined array.
#define FMEDIA_STATIC_ARRAY_COUNT(array)			(SIZE_T)(sizeof((array)) / sizeof((array)[0]))

//--------------------------------------------------------------------------
//
// Endian swapping
//
static inline uint16 MEDIA_ENDIAN_SWAP(uint16 value)
{
	return((value >> 8) | (value << 8));
}

static inline int16 MEDIA_ENDIAN_SWAP(int16 value)
{
	return(int16(MEDIA_ENDIAN_SWAP(uint16(value))));
}

static inline uint32 MEDIA_ENDIAN_SWAP(uint32 value)
{
	return((value << 24) | ((value & 0xff00) << 8) | ((value >> 8) & 0xff00) | (value >> 24));
}

static inline int32 MEDIA_ENDIAN_SWAP(int32 value)
{
	return(int32(MEDIA_ENDIAN_SWAP(uint32(value))));
}

static inline uint64 MEDIA_ENDIAN_SWAP(uint64 value)
{
	return((uint64(MEDIA_ENDIAN_SWAP(uint32(value & 0xffffffffU))) << 32) | uint64(MEDIA_ENDIAN_SWAP(uint32(value >> 32))));
}

static inline int64 MEDIA_ENDIAN_SWAP(int64 value)
{
	return(int64(MEDIA_ENDIAN_SWAP(uint64(value))));
}


// Yes, there's no endian swapping with 8 bit values. We put them here so templated functions can be used with these types.
static inline uint8  MEDIA_TO_BIG_ENDIAN(uint8 value)			{ return(value); }
static inline int8   MEDIA_TO_BIG_ENDIAN(int8 value)			{ return(value); }
static inline uint8  MEDIA_TO_LITTLE_ENDIAN(uint8 value)		{ return(value); }
static inline int8   MEDIA_TO_LITTLE_ENDIAN(int8 value)			{ return(value); }
static inline uint8  MEDIA_FROM_BIG_ENDIAN(uint8 value)			{ return(value); }
static inline int8   MEDIA_FROM_BIG_ENDIAN(int8 value)			{ return(value); }
static inline uint8  MEDIA_FROM_LITTLE_ENDIAN(uint8 value)		{ return(value); }
static inline int8   MEDIA_FROM_LITTLE_ENDIAN(int8 value)		{ return(value); }


#if !PLATFORM_LITTLE_ENDIAN
// Convert a native value to big endian
static inline uint16 MEDIA_TO_BIG_ENDIAN(uint16 value)		{ return(value); }
static inline int16 MEDIA_TO_BIG_ENDIAN(int16 value)		{ return(value); }
static inline int32 MEDIA_TO_BIG_ENDIAN(int32 value)		{ return(value); }
static inline uint32 MEDIA_TO_BIG_ENDIAN(uint32 value)		{ return(value); }
static inline int64 MEDIA_TO_BIG_ENDIAN(int64 value)		{ return(value); }
static inline uint64 MEDIA_TO_BIG_ENDIAN(uint64 value)		{ return(value); }
// Convert a native value to little endian
static inline uint16 MEDIA_TO_LITTLE_ENDIAN(uint16 value)	{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline int16 MEDIA_TO_LITTLE_ENDIAN(int16 value)		{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline int32 MEDIA_TO_LITTLE_ENDIAN(int32 value)		{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline uint32 MEDIA_TO_LITTLE_ENDIAN(uint32 value)	{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline int64 MEDIA_TO_LITTLE_ENDIAN(int64 value)		{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline uint64 MEDIA_TO_LITTLE_ENDIAN(uint64 value)	{ return(MEDIA_ENDIAN_SWAP(value)); }
// Convert a big endian value to native
static inline uint16 MEDIA_FROM_BIG_ENDIAN(uint16 value)	{ return(value); }
static inline int16 MEDIA_FROM_BIG_ENDIAN(int16 value)		{ return(value); }
static inline int32 MEDIA_FROM_BIG_ENDIAN(int32 value)		{ return(value); }
static inline uint32 MEDIA_FROM_BIG_ENDIAN(uint32 value)	{ return(value); }
static inline int64 MEDIA_FROM_BIG_ENDIAN(int64 value)		{ return(value); }
static inline uint64 MEDIA_FROM_BIG_ENDIAN(uint64 value)	{ return(value); }
// Convert a little endian value to native
static inline uint16 MEDIA_FROM_LITTLE_ENDIAN(uint16 value)	{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline int16 MEDIA_FROM_LITTLE_ENDIAN(int16 value)	{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline int32 MEDIA_FROM_LITTLE_ENDIAN(int32 value)	{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline uint32 MEDIA_FROM_LITTLE_ENDIAN(uint32 value)	{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline int64 MEDIA_FROM_LITTLE_ENDIAN(int64 value)	{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline uint64 MEDIA_FROM_LITTLE_ENDIAN(uint64 value)	{ return(MEDIA_ENDIAN_SWAP(value)); }

#else

// Convert a native value to big endian
static inline uint16 MEDIA_TO_BIG_ENDIAN(uint16 value)		{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline int16 MEDIA_TO_BIG_ENDIAN(int16 value)		{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline int32 MEDIA_TO_BIG_ENDIAN(int32 value)		{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline uint32 MEDIA_TO_BIG_ENDIAN(uint32 value)		{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline int64 MEDIA_TO_BIG_ENDIAN(int64 value)		{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline uint64 MEDIA_TO_BIG_ENDIAN(uint64 value)		{ return(MEDIA_ENDIAN_SWAP(value)); }
// Convert a native value to little endian
static inline uint16 MEDIA_TO_LITTLE_ENDIAN(uint16 value)	{ return(value); }
static inline int16 MEDIA_TO_LITTLE_ENDIAN(int16 value)		{ return(value); }
static inline int32 MEDIA_TO_LITTLE_ENDIAN(int32 value)		{ return(value); }
static inline uint32 MEDIA_TO_LITTLE_ENDIAN(uint32 value)	{ return(value); }
static inline int64 MEDIA_TO_LITTLE_ENDIAN(int64 value)		{ return(value); }
static inline uint64 MEDIA_TO_LITTLE_ENDIAN(uint64 value)	{ return(value); }
// Convert a big endian value to native
static inline uint16 MEDIA_FROM_BIG_ENDIAN(uint16 value)	{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline int16 MEDIA_FROM_BIG_ENDIAN(int16 value)		{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline int32 MEDIA_FROM_BIG_ENDIAN(int32 value)		{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline uint32 MEDIA_FROM_BIG_ENDIAN(uint32 value)	{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline int64 MEDIA_FROM_BIG_ENDIAN(int64 value)		{ return(MEDIA_ENDIAN_SWAP(value)); }
static inline uint64 MEDIA_FROM_BIG_ENDIAN(uint64 value)	{ return(MEDIA_ENDIAN_SWAP(value)); }
// Convert a little endian value to native
static inline uint16 MEDIA_FROM_LITTLE_ENDIAN(uint16 value)	{ return(value); }
static inline int16 MEDIA_FROM_LITTLE_ENDIAN(int16 value)	{ return(value); }
static inline int32 MEDIA_FROM_LITTLE_ENDIAN(int32 value)	{ return(value); }
static inline uint32 MEDIA_FROM_LITTLE_ENDIAN(uint32 value)	{ return(value); }
static inline int64 MEDIA_FROM_LITTLE_ENDIAN(int64 value)	{ return(value); }
static inline uint64 MEDIA_FROM_LITTLE_ENDIAN(uint64 value)	{ return(value); }

#endif
