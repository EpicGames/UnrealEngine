// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/MediaTypes.h"
#include "Core/MediaMacros.h"
#include "Core/MediaNoncopyable.h"
#include "Core/MediaSemaphore.h"
#include "Core/MediaQueue.h"


/**
 * Generic message queue WITHOUT timeout support.
 *
 * Conceptually a message queue is a multiple-sender, single-receiver object, so use it only in that way!
 *
 * This version supports only a maximum number of messages that needs to be specified in the constructor
 * or by calling Resize() before first use.
 */
template <typename T>
class TMediaMessageQueueNoTimeout
{
public:
	// ----------------------------------------------------------------------------
	/**
	 * Constructor.
	 */
	TMediaMessageQueueNoTimeout(SIZE_T MaxMessages = 0)
		: Messages(MaxMessages)
		, AvailSema(MaxMessages)
	{
	}

	// ----------------------------------------------------------------------------
	/**
	 * Waits indefinitely for a message to arrive.
	 *
	 * @return Received message
	 */
	T ReceiveMessage()
	{
		ReadySema.Obtain();
		T Msg = Messages.Pop();
		AvailSema.Release();
		return Msg;
	}

	// ----------------------------------------------------------------------------
	/**
	 * Receives a message if one is already pending in the queue.
	 *
	 * @param message Receives the message if true is returned, unchanged otherwise
	 *
	 * @return true if message received and stored in 'message', false if no message received.
	 */
	bool ReceiveMessage(T& Message)
	{
		bool bHave;
		bHave = ReadySema.TryToObtain();
		if (bHave)
		{
			Message = Messages.Pop();
			AvailSema.Release();
			return true;
		}
		return false;
	}

	// ----------------------------------------------------------------------------
	/**
	 * Sends a message.
	 *
	 * @param message Message to send
	 * @param bWaitIfFull true to wait for the queue to have room if it's full, false to return immediately.
	 *
	 * @return true if message sent successfully, false if queue is full and bWaitIfFull was false.
	 */
	bool SendMessage(const T& Message, bool bWaitIfFull = true)
	{
		bool bOk = bWaitIfFull ? AvailSema.Obtain() : AvailSema.TryToObtain();
		if (bOk)
		{
			Messages.Push(Message);
			ReadySema.Release();
			return true;
		}
		return false;
	}

	// ----------------------------------------------------------------------------
	/**
	 * Jams a message to the front of the queue.
	 *
	 * @param message Message to send
	 * @param bWaitIfFull true to wait for the queue to have room if it's full, false to return immediately.
	 *
	 * @return true if message sent successfully, false if queue is full and bWaitIfFull was false.
	 */
	bool JamMessage(const T& Message, bool bWaitIfFull = true)
	{
		bool bOk = bWaitIfFull ? AvailSema.Obtain() : AvailSema.TryToObtain();
		if (bOk)
		{
			Messages.PushFront(Message);
			ReadySema.Release();
			return true;
		}
		return false;
	}

	// ----------------------------------------------------------------------------
	/**
	 * Checks if there are pending messages.
	 *
	 * @return true if there are pending messages, false if not.
	 */
	bool HaveMessage() const
	{
		return !Messages.IsEmpty();
	}

	// ----------------------------------------------------------------------------
	/**
	 * Queries the number of messages in the queue.
	 *
	 * @return Number of enqueued messages.
	 */
	SIZE_T NumWaitingMessages() const
	{
		return Messages.Num();
	}

	// ----------------------------------------------------------------------------
	/**
	 * Resizes the message queue, discarding any messages it may hold.
	 *
	 * Should only be used after the queue was created, but before it gets used!
	 *
	 * @param maxMessages New message queue size.
	 *
	 * @return none
	 */
	void Resize(SIZE_T MaxMessages)
	{
		ReadySema.SetCount(0);
		AvailSema.SetCount(MaxMessages);
		Messages.Resize(MaxMessages);
	}

private:
	TMediaQueue<T, FMediaLockCriticalSection> Messages;
	FMediaSemaphore AvailSema;
	FMediaSemaphore ReadySema;
};




/**
 * Generic message queue WITH timeout support.
 *
 * Conceptually a message queue is a multiple-sender, single-receiver object, so use it only in that way!
 *
 * This version supports only a maximum number of messages that needs to be specified in the constructor
 * or by calling Resize() before first use.
 */
template <typename T>
class TMediaMessageQueueWithTimeout
{
public:
	// ----------------------------------------------------------------------------
	/**
	 * Constructor.
	 */
	TMediaMessageQueueWithTimeout(SIZE_T MaxMessages = 0)
		: Messages(MaxMessages)
		, AvailSema(MaxMessages)
	{
	}

	// ----------------------------------------------------------------------------
	/**
	 * Waits indefinitely for a message to arrive.
	 *
	 * @return Received message
	 */
	T ReceiveMessage()
	{
		ReadySema.Obtain();
		T Msg = Messages.Pop();
		AvailSema.Release();
		return Msg;
	}

	// ----------------------------------------------------------------------------
	/**
	 * Receives a message if one is already pending in the queue.
	 *
	 * @param message Receives the message if true is returned, unchanged otherwise
	 *
	 * @return true if message received and stored in 'message', false if no message received.
	 */
	bool ReceiveMessage(T& Message)
	{
		bool bHave;
		bHave = ReadySema.TryToObtain();
		if (bHave)
		{
			Message = Messages.Pop();
			AvailSema.Release();
			return true;
		}
		return false;
	}

	// ----------------------------------------------------------------------------
	/**
	 * Waits a maximum amount of time for a message to arrive.
	 *
	 * @param message Receives the message if true is returned, unchanged otherwise
	 * @param maxWaitMicroseconds Maximum wait time in microseconds
	 *
	 * @return true if message received and stored in 'message', false if no message received.
	 */
	bool ReceiveMessage(T& Message, int64 MaxWaitMicroseconds)
	{
		check(MaxWaitMicroseconds >= 0);
		bool bHave;
		bHave = ReadySema.Obtain(MaxWaitMicroseconds);
		if (bHave)
		{
			Message = Messages.Pop();
			AvailSema.Release();
			return true;
		}
		return false;
	}

	// ----------------------------------------------------------------------------
	/**
	 * Sends a message.
	 *
	 * @param message Message to send
	 * @param bWaitIfFull true to wait for the queue to have room if it's full, false to return immediately.
	 *
	 * @return true if message sent successfully, false if queue is full and bWaitIfFull was false.
	 */
	bool SendMessage(const T& Message, bool bWaitIfFull = true)
	{
		bool bOk = bWaitIfFull ? AvailSema.Obtain() : AvailSema.TryToObtain();
		if (bOk)
		{
			Messages.Push(Message);
			ReadySema.Release();
			return true;
		}
		return false;
	}

	// ----------------------------------------------------------------------------
	/**
	 * Jams a message to the front of the queue.
	 *
	 * @param message Message to send
	 * @param bWaitIfFull true to wait for the queue to have room if it's full, false to return immediately.
	 *
	 * @return true if message sent successfully, false if queue is full and bWaitIfFull was false.
	 */
	bool JamMessage(const T& Message, bool bWaitIfFull = true)
	{
		bool bOk = bWaitIfFull ? AvailSema.Obtain() : AvailSema.TryToObtain();
		if (bOk)
		{
			Messages.PushFront(Message);
			ReadySema.Release();
			return true;
		}
		return false;
	}

	// ----------------------------------------------------------------------------
	/**
	 * Checks if there are pending messages.
	 *
	 * @return true if there are pending messages, false if not.
	 */
	bool HaveMessage() const
	{
		return !Messages.IsEmpty();
	}

	// ----------------------------------------------------------------------------
	/**
	 * Queries the number of messages in the queue.
	 *
	 * @return Number of enqueued messages.
	 */
	SIZE_T NumWaitingMessages() const
	{
		return Messages.Num();
	}

	// ----------------------------------------------------------------------------
	/**
	 * Resizes the message queue, discarding any messages it may hold.
	 *
	 * Should only be used after the queue was created, but before it gets used!
	 *
	 * @param maxMessages New message queue size.
	 *
	 * @return none
	 */
	void Resize(SIZE_T MaxMessages)
	{
		ReadySema.SetCount(0);
		AvailSema.SetCount(MaxMessages);
		Messages.Resize(MaxMessages);
	}

private:
	TMediaQueue<T, FMediaLockCriticalSection> Messages;
	FMediaSemaphore AvailSema;
	FMediaSemaphore ReadySema;
};





/**
 * Generic message queue WITHOUT timeout support.
 *
 * Conceptually a message queue is a multiple-sender, single-receiver object, so use it only in that way!
 *
 * This version uses a template argument to define the initial size.
**/
template <typename T, SIZE_T CAPACITY>
class TMediaMessageQueueStaticNoTimeout : public TMediaMessageQueueNoTimeout<T>
{
public:
	// ----------------------------------------------------------------------------
	/**
	 * Constructor.
	**/
	TMediaMessageQueueStaticNoTimeout()
		: TMediaMessageQueueNoTimeout<T>(CAPACITY)
	{
	}
};


/**
 * Generic message queue WITH timeout support.
 *
 * Conceptually a message queue is a multiple-sender, single-receiver object, so use it only in that way!
 *
 * This version uses a template argument to define the initial size.
**/
template <typename T, SIZE_T CAPACITY>
class TMediaMessageQueueStaticWithTimeout : public TMediaMessageQueueWithTimeout<T>
{
public:
	// ----------------------------------------------------------------------------
	/**
	 * Constructor.
	**/
	TMediaMessageQueueStaticWithTimeout()
		: TMediaMessageQueueWithTimeout<T>(CAPACITY)
	{
	}
};










/**
 * Generic message queue WITHOUT timeout support.
 *
 * Conceptually a message queue is a multiple-sender, single-receiver object, so use it only in that way!
 *
 * This version supports an unlimited number of messages (we limit it to 2^31-1)
 * Sending a message can therefore never block. We keep the interface identical to TMediaMessageQueueNoTimeout::SendMessage
 * for compatibility.
**/
template <typename T>
class TMediaMessageQueueDynamicNoTimeout
{
public:
	// ----------------------------------------------------------------------------
	/**
	 * Constructor.
	**/
	TMediaMessageQueueDynamicNoTimeout()
		: AvailSema(0x7fffffff)
	{
	}

	// ----------------------------------------------------------------------------
	/**
	 * Waits indefinitely for a message to arrive.
	 *
	 * @return		Received message
	**/
	T ReceiveMessage()
	{
		ReadySema.Obtain();
		T Msg = Messages.Pop();
		AvailSema.Release();
		return Msg;
	}

	// ----------------------------------------------------------------------------
	/**
	 * Receives a message if one is already pending in the queue.
	 *
	 * @param		message     Receives the message if true is returned, unchanged otherwise
	 *
	 * @return		true if message received and stored in 'message', false if no message received.
	**/
	bool ReceiveMessage(T& Message)
	{
		bool bHave;
		bHave = ReadySema.TryToObtain();
		if (bHave)
		{
			Message = Messages.Pop();
			AvailSema.Release();
			return true;
		}
		return false;
	}

	// ----------------------------------------------------------------------------
	/**
	 * Sends a message.
	 *
	 * @param		message         Message to send
	 * @param		bWaitIfFull     true to wait for the queue to have room if it's full, false to return immediately.
	 *
	 * @return		true if message sent successfully, false if queue is full and bWaitIfFull was false.
	**/
	bool SendMessage(const T& Message, bool bWaitIfFull = true)
	{
		bool bOk = bWaitIfFull ? AvailSema.Obtain() : AvailSema.TryToObtain();
		if (bOk)
		{
			Messages.Push(Message);
			ReadySema.Release();
			return true;
		}
		return false;
	}

	// ----------------------------------------------------------------------------
	/**
	 * Jams a message to the front of the queue.
	 *
	 * @param		message         Message to send
	 * @param		bWaitIfFull     true to wait for the queue to have room if it's full, false to return immediately.
	 *
	 * @return		true if message sent successfully, false if queue is full and bWaitIfFull was false.
	**/
	bool JamMessage(const T& Message, bool bWaitIfFull = true)
	{
		bool bOk = bWaitIfFull ? AvailSema.Obtain() : AvailSema.TryToObtain();
		if (bOk)
		{
			Messages.PushFront(Message);
			ReadySema.Release();
			return true;
		}
		return false;
	}

	// ----------------------------------------------------------------------------
	/**
	 * Checks if there are pending messages.
	 *
	 * @return		true if there are pending messages, false if not.
	**/
	bool HaveMessage() const
	{
		return !Messages.IsEmpty();
	}

	// ----------------------------------------------------------------------------
	/**
	 * Queries the number of messages in the queue.
	 *
	 * @return		Number of enqueued messages.
	**/
	SIZE_T NumWaitingMessages() const
	{
		return Messages.Num();
	}

private:
	TMediaQueueDynamic<T>	Messages;
	FMediaSemaphore			AvailSema;
	FMediaSemaphore			ReadySema;
};






/**
 * Generic message queue WITH timeout support.
 *
 * Conceptually a message queue is a multiple-sender, single-receiver object, so use it only in that way!
 *
 * This version supports an unlimited number of messages (we limit it to 2^31-1)
 * Sending a message can therefore never block. We keep the interface identical to TMediaMessageQueueNoTimeout::SendMessage
 * for compatibility.
**/
template <typename T>
class TMediaMessageQueueDynamicWithTimeout
{
public:
	// ----------------------------------------------------------------------------
	/**
	 * Constructor.
	**/
	TMediaMessageQueueDynamicWithTimeout()
		: AvailSema(0x7fffffff)
	{
	}

	// ----------------------------------------------------------------------------
	/**
	 * Waits indefinitely for a message to arrive.
	 *
	 * @return		Received message
	**/
	T ReceiveMessage()
	{
		ReadySema.Obtain();
		T Msg = Messages.Pop();
		AvailSema.Release();
		return Msg;
	}

	// ----------------------------------------------------------------------------
	/**
	 * Receives a message if one is already pending in the queue.
	 *
	 * @param		message     Receives the message if true is returned, unchanged otherwise
	 *
	 * @return		true if message received and stored in 'message', false if no message received.
	**/
	bool ReceiveMessage(T& Message)
	{
		bool bHave;
		bHave = ReadySema.TryToObtain();
		if (bHave)
		{
			Message = Messages.Pop();
			AvailSema.Release();
			return true;
		}
		return false;
	}

	// ----------------------------------------------------------------------------
	/**
	 * Waits a maximum amount of time for a message to arrive.
	 *
	 * @param		message                 Receives the message if true is returned, unchanged otherwise
	 * @param		maxWaitMicroseconds     Maximum wait time in microseconds
	 *
	 * @return		true if message received and stored in 'message', false if no message received.
	**/
	bool ReceiveMessage(T& Message, int64 MaxWaitMicroseconds)
	{
		check(MaxWaitMicroseconds >= 0);
		bool bHave;
		bHave = ReadySema.Obtain(MaxWaitMicroseconds);
		if (bHave)
		{
			Message = Messages.Pop();
			AvailSema.Release();
			return true;
		}
		return false;
	}

	// ----------------------------------------------------------------------------
	/**
	 * Sends a message.
	 *
	 * @param		message         Message to send
	 * @param		bWaitIfFull     true to wait for the queue to have room if it's full, false to return immediately.
	 *
	 * @return		true if message sent successfully, false if queue is full and bWaitIfFull was false.
	**/
	bool SendMessage(const T& Message, bool bWaitIfFull = true)
	{
		bool bOk = bWaitIfFull ? AvailSema.Obtain() : AvailSema.TryToObtain();
		if (bOk)
		{
			Messages.Push(Message);
			ReadySema.Release();
			return true;
		}
		return false;
	}

	// ----------------------------------------------------------------------------
	/**
	 * Jams a message to the front of the queue.
	 *
	 * @param		message         Message to send
	 * @param		bWaitIfFull     true to wait for the queue to have room if it's full, false to return immediately.
	 *
	 * @return		true if message sent successfully, false if queue is full and bWaitIfFull was false.
	**/
	bool JamMessage(const T& Message, bool bWaitIfFull = true)
	{
		bool bOk = bWaitIfFull ? AvailSema.Obtain() : AvailSema.TryToObtain();
		if (bOk)
		{
			Messages.PushFront(Message);
			ReadySema.Release();
			return true;
		}
		return false;
	}

	// ----------------------------------------------------------------------------
	/**
	 * Checks if there are pending messages.
	 *
	 * @return		true if there are pending messages, false if not.
	**/
	bool HaveMessage() const
	{
		return !Messages.IsEmpty();
	}

	// ----------------------------------------------------------------------------
	/**
	 * Queries the number of messages in the queue.
	 *
	 * @return		Number of enqueued messages.
	**/
	SIZE_T NumWaitingMessages() const
	{
		return Messages.Num();
	}

private:
	TMediaQueueDynamic<T>			Messages;
	FMediaSemaphore					AvailSema;
	FMediaSemaphore					ReadySema;
};



