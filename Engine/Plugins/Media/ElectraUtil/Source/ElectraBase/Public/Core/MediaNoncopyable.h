// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

template<typename T>
class TMediaNoncopyable
{
protected:
	TMediaNoncopyable() = default;
	~TMediaNoncopyable() = default;
private:
	TMediaNoncopyable(const TMediaNoncopyable&) = delete;
	TMediaNoncopyable& operator = (const TMediaNoncopyable&) = delete;
};
