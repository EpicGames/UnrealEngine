// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/MediaTypes.h"
#include "Core/MediaMacros.h"
#include "Core/MediaLock.h"
#include "Core/MediaInterlocked.h"
#include "Core/MediaNoncopyable.h"


/**
 * Default memory allocator.
 */
template <typename T>
class TMediaQueueAllocator
{
public:
	static T* Allocate(uint32 NumElements)
	{
		return reinterpret_cast<T*>(FMemory::Malloc(NumElements * sizeof(T),  __alignof(T)));
	}
	static void Deallocate(T* Address)
	{
		if (Address)
		{
			FMemory::Free((void*)Address);
		}
	}
};



/**
 * Object safe queue class.
 *
 * Implements by-value methods only to avoid working with references.
 *
 * Methods like
 *  T& Push()
 *  T& operator[](int32)
 *  T& Front()
 *
 * are dangerous when used in a multi-threaded environment since the elements cannot be protected by an internal mutex.
 * If you are using an external mutex to ensure object safety you may add such functions, but please derive a new class for this!!!
 *
 */
template <typename T, typename L = FMediaLockNone, typename M = TMediaQueueAllocator<T> >
class TMediaQueue : public L
{
public:
	using MyType = TMediaQueue<T, L, M>;
	using ElementType = T;

	TMediaQueue(int32 InitialCapacity = 0)
	{
		Resize(InitialCapacity);
	}

	// Not copyable (for now) because of the possible mutex class.
	TMediaQueue(const MyType& rhs) = delete;
	MyType& operator = (const MyType& rhs) = delete;

	~TMediaQueue()
	{
		Resize(0);
	}

	void Clear()
	{
		L::Lock();
		for(int32 i=0; i<NumIn; ++i)
		{
			Elements[IdxOut].~T();
			if (++IdxOut == MaxNum)
			{
				IdxOut -= MaxNum;
			}
		}
		IdxIn = IdxOut = NumIn = 0;
		L::Unlock();
	}

	void Resize(int32 newCapacity)
	{
		L::Lock();
		Clear();
		if (newCapacity != MaxNum)
		{
			M::Deallocate(Elements);
			if (newCapacity)
			{
				Elements = M::Allocate(newCapacity);
			}
			else
			{
				Elements = nullptr;
			}
			MaxNum = newCapacity;
		}
		L::Unlock();
	}

	void Swap(MyType& rhs)
	{
		L::Lock();
		// Remember our values.
		T* pElements = Elements;
		int32  nCapacity = MaxNum;
		int32  nNumIn = NumIn;
		int32  nIdxIn = IdxIn;
		int32  nIdxOut = IdxOut;
		// Get the values from the other queue.
		Elements = rhs.Elements;
		MaxNum = rhs.MaxNum;
		NumIn = rhs.NumIn;
		IdxIn = rhs.IdxIn;
		IdxOut = rhs.IdxOut;
		// Set our old values in the other queue.
		rhs.Elements = pElements;
		rhs.MaxNum = nCapacity;
		rhs.NumIn = nNumIn;
		rhs.IdxIn = nIdxIn;
		rhs.IdxOut = nIdxOut;
		L::Unlock();
	}

	int32 Capacity() const
	{
		return MaxNum;
	}

	int32 Num() const
	{
		return NumIn;
	}

	bool IsEmpty() const
	{
		return Num() == 0;
	}

	bool IsFull() const
	{
		return Num() == Capacity();
	}

	void Push(const T& Element)
	{
		L::Lock();
		check(!IsFull());
		int32 pos = IdxIn;
		if (++IdxIn == MaxNum)
		{
			IdxIn = 0;
		}
		++NumIn;
		// Copying the element can NOT take place outside the lock in case multiple threads push at the same time and a Resize() will happen.
		new ((void*)(Elements + pos)) T(Element);
		L::Unlock();
	}

	void PushFront(const T& Element)
	{
		L::Lock();
		check(!IsFull());
		if (IdxOut == 0)
		{
			IdxOut = MaxNum - 1;
		}
		else
		{
			--IdxOut;
		}
		int32 pos = IdxOut;
		++NumIn;
		// Copying the element can NOT take place outside the lock in case multiple threads push at the same time and a Resize() will happen.
		new ((void*)(Elements + pos)) T(Element);
		L::Unlock();
	}

	T Pop()
	{
		L::Lock();
		check(!IsEmpty());
		T r(Elements[IdxOut]);
		Elements[IdxOut].~T();
		if (++IdxOut == MaxNum)
		{
			IdxOut = 0;
		}
		--NumIn;
		L::Unlock();
		return r;
	}

	T Front() const
	{
		L::Lock();
		check(!IsEmpty());
		T r(Elements[IdxOut]);
		L::Unlock();
		return r;
	}

protected:
	T* Elements = nullptr;
	int32 MaxNum = 0;
	int32 NumIn = 0;
	int32 IdxIn = 0;
	int32 IdxOut = 0;
};





/**
 * A mutex-less version of TMediaQueue to enable working with references.
 *
 * Since this is not thread-safe you must ensure access with an external mutex.
 */
template <typename T, typename M = TMediaQueueAllocator<T> >
class TMediaQueueNoLock : public TMediaQueue<T, FMediaLockNone, M>
{
public:
	using MyType = TMediaQueueNoLock<T, M>;
	using ElementType = T;
	typedef TMediaQueue<T, FMediaLockNone, M>	B;

	TMediaQueueNoLock() : B() {}
	~TMediaQueueNoLock() = default;

	T& PushRef()
	{
		check(!B::IsFull());
		int32 pos = B::IdxIn;
		if (++B::IdxIn == B::Capacity())
		{
			B::IdxIn = 0;
		}
		++B::NumIn;
		new ((void*)(B::Elements + pos)) T();
		return B::Elements[pos];
	}

	const T& FrontRef() const
	{
		check(!B::IsEmpty());
		return B::Elements[B::IdxOut];
	}

	T& FrontRef()
	{
		check(!B::IsEmpty());
		return B::Elements[B::IdxOut];
	}

	const T& BackRef() const
	{
		check(!B::IsEmpty());
		return B::Elements[(B::IdxIn + B::Capacity() - 1) % B::Capacity()];
	}

	T& BackRef()
	{
		check(!B::IsEmpty());
		return B::Elements[(B::IdxIn + B::Capacity() - 1) % B::Capacity()];
	}

	void Erase(int32 i)
	{
		check(!B::IsEmpty());
		check(i < B::NumIn);
		--B::NumIn;
		int32 i0 = (i + B::IdxOut) % B::Capacity();
		int32 i1 = (i + B::IdxOut + 1) % B::Capacity();
		for(; i < B::NumIn; ++i)
		{
			B::Elements[i0] = B::Elements[i1];
			i0 = i1++;
			if (i1 >= B::Capacity())
			{
				i1 -= B::Capacity();
			}
		}
		B::Elements[i0].~T();
		if (B::IdxIn == 0)
		{
			B::IdxIn = B::Capacity() - 1;
		}
		else
		{
			--B::IdxIn;
		}
	}

	const T& operator[](int32 i) const
	{
		check(!B::IsEmpty());
		check(i < B::NumIn);
		i += B::IdxOut;
		if (i >= B::Capacity())
		{
			i -= B::Capacity();
		}
		return B::Elements[i];
	}

	T& operator[](int32 i)
	{
		check(!B::IsEmpty());
		check(i < B::NumIn);
		i += B::IdxOut;
		if (i >= B::Capacity())
		{
			i -= B::Capacity();
		}
		return B::Elements[i];
	}

	void AppendFirstElements(const MyType& FromOther, int32 MaxElements = ~int32(0))
	{
		// NOTE: Your code needs to make sure the source queue is protected by some external lock so does not get
		//       modified while we are appending its elements.
		int32 NumAvail = B::Capacity() - B::Num();
		int32 NumToAppend = FromOther.Num();
		if (NumToAppend > NumAvail)
		{
			NumToAppend = NumAvail;
		}
		if (NumToAppend > MaxElements)
		{
			NumToAppend = MaxElements;
		}
		for(int32 i = 0; i < NumToAppend; ++i)
		{
			B::Push(FromOther[i]);
		}
	}

	void AppendLastElements(const MyType& FromOther, int32 MaxElements = ~int32(0))
	{
		// NOTE: Your code needs to make sure the source queue is protected by some external lock so does not get
		//       modified while we're appending its elements.
		int32 NumAvail = B::Capacity() - B::Num();
		int32 NumToAppend = FromOther.Num();
		if (NumToAppend > NumAvail)
		{
			NumToAppend = NumAvail;
		}
		if (NumToAppend > MaxElements)
		{
			NumToAppend = MaxElements;
		}
		for(int32 i = 0, j = FromOther.Num() - NumToAppend; i < NumToAppend; ++i, ++j)
		{
			B::Push(FromOther[j]);
		}
	}

};





/**
 * Object safe queue class.
 *
 * Implements by-value methods only to avoid working with references.
 *
 * Methods like
 *  T& Push()
 *  T& operator[](int32)
 *  T& Front()
 *
 * are dangerous when used in a multi-threaded environment since the elements cannot be protected by an internal mutex.
 * If you are using an external mutex to ensure object safety you may add such functions, but please derive a new class for this!!!
 * (or use the one below)
 *
 */
template <typename T, uint32 CAPACITY, typename L = FMediaLockCriticalSection >
class TMediaQueueFixedStatic : public L
{
public:
	using MyType = TMediaQueueFixedStatic<T, CAPACITY, L>;
	using ElementType = T;

	TMediaQueueFixedStatic() = default;

	// Not copyable (for now) because of the possible mutex class.
	TMediaQueueFixedStatic(const MyType& rhs) = delete;
	MyType& operator = (const MyType& rhs) = delete;

	~TMediaQueueFixedStatic()
	{
		Clear();
	}

	void Clear()
	{
		L::Lock();
		for(int32 i=0; i<NumIn; ++i)
		{
			Elements[IdxOut].~T();
			if (++IdxOut == CAPACITY)
			{
				IdxOut -= CAPACITY;
			}
		}
		IdxIn = IdxOut = NumIn = 0;
		L::Unlock();
	}

	int32 Capacity() const
	{
		return CAPACITY;
	}

	int32 Num() const
	{
		return NumIn;
	}

	bool IsEmpty() const
	{
		return Num() == 0;
	}

	bool IsFull() const
	{
		return Num() == Capacity();
	}

	void Push(const T& element)
	{
		L::Lock();
		check(!IsFull());
		int32 pos = IdxIn;
		if (++IdxIn == CAPACITY)
		{
			IdxIn = 0;
		}
		++NumIn;
		// Copying the element can NOT take place outside the lock in case multiple threads push at the same time and a Resize() will happen.
		new ((void*)(Elements + pos)) T(element);
		L::Unlock();
	}

	void PushFront(const T& element)
	{
		L::Lock();
		check(!IsFull());
		if (IdxOut == 0)
		{
			IdxOut = CAPACITY - 1;
		}
		else
		{
			--IdxOut;
		}
		int32 pos = IdxOut;
		++NumIn;
		// Copying the element can NOT take place outside the lock in case multiple threads push at the same time and a Resize() will happen.
		new ((void*)(Elements + pos)) T(element);
		L::Unlock();
	}

	T Pop()
	{
		L::Lock();
		check(!IsEmpty());
		T r(Elements[IdxOut]);
		Elements[IdxOut].~T();
		if (++IdxOut == CAPACITY)
		{
			IdxOut = 0;
		}
		--NumIn;
		L::Unlock();
		return r;
	}

	T Front() const
	{
		L::Lock();
		check(!IsEmpty());
		T r(Elements[IdxOut]);
		L::Unlock();
		return r;
	}

protected:

	MS_ALIGN(16)
	uint8 FixedElements[CAPACITY * sizeof(T)]
	GCC_ALIGN(16);
	T* Elements = (T*)FixedElements;
	int32 NumIn = 0;
	int32 IdxIn = 0;
	int32 IdxOut = 0;
};



template <typename T, uint32 CAPACITY>
class TMediaQueueFixedStaticNoLock : public TMediaQueueFixedStatic<T, CAPACITY, FMediaLockNone>
{
public:
	using MyType = TMediaQueueFixedStaticNoLock<T, CAPACITY>;
	using ElementType = T;
	typedef TMediaQueueFixedStatic<T, CAPACITY, FMediaLockNone>	B;

	TMediaQueueFixedStaticNoLock() : B() {}
	~TMediaQueueFixedStaticNoLock() = default;

	T& PushRef()
	{
		check(!B::IsFull());
		int32 pos = B::IdxIn;
		if (++B::IdxIn == CAPACITY)
		{
			B::IdxIn = 0;
		}
		++B::NumIn;
		new ((void*)(B::Elements + pos)) T();
		return B::Elements[pos];
	}

	const T& FrontRef() const
	{
		check(!B::IsEmpty());
		return B::Elements[B::IdxOut];
	}

	T& FrontRef()
	{
		check(!B::IsEmpty());
		return B::Elements[B::IdxOut];
	}

	const T& BackRef() const
	{
		check(!B::IsEmpty());
		return B::Elements[(B::IdxIn + B::Capacity() - 1) % B::Capacity()];
	}

	T& BackRef()
	{
		check(!B::IsEmpty());
		return B::Elements[(B::IdxIn + B::Capacity() - 1) % B::Capacity()];
	}

	void Erase(int32 i)
	{
		check(!B::IsEmpty());
		check(i < B::NumIn);
		--B::NumIn;
		int32 i0 = (i + B::IdxOut) % CAPACITY;
		int32 i1 = (i + B::IdxOut + 1) % CAPACITY;
		for(; i < B::NumIn; ++i)
		{
			B::Elements[i0] = B::Elements[i1];
			i0 = i1++;
			if (i1 >= CAPACITY)
			{
				i1 -= CAPACITY;
			}
		}
		B::Elements[i0].~T();
		if (B::IdxIn == 0)
		{
			B::IdxIn = CAPACITY - 1;
		}
		else
		{
			--B::IdxIn;
		}
	}

	const T& operator[](int32 i) const
	{
		check(!B::IsEmpty());
		check(i < B::NumIn);
		i += B::IdxOut;
		if (i >= CAPACITY)
		{
			i -= CAPACITY;
		}
		return B::Elements[i];
	}

	T& operator[](int32 i)
	{
		check(!B::IsEmpty());
		check(i < B::NumIn);
		i += B::IdxOut;
		if (i >= CAPACITY)
		{
			i -= CAPACITY;
		}
		return B::Elements[i];
	}
};





/**
 * Dynamically sized queue.
 *
 * Will grow when capacity is reached.
 * Therefore a lock is REQUIRED! Never use this with a null-mutex unless you know what you're doing.
 */
template <typename T, typename L = FMediaLockCriticalSection, typename M = TMediaQueueAllocator<T> >
class TMediaQueueDynamic : public L, private TMediaNoncopyable<TMediaQueueDynamic<T> >
{
public:
	using MyType = TMediaQueueDynamic<T, L, M>;
	using ElementType = T;

	TMediaQueueDynamic(int32 initialCapacity = 0, int32 increment = 32)
		: IncrementBy(increment)
	{
		Resize(initialCapacity);
	}

	~TMediaQueueDynamic()
	{
		L::Lock();
		Resize(0);
		L::Unlock();
	}

	void SetIncrement(int32 newIncrement)
	{
		L::Lock();
		IncrementBy = newIncrement;
		L::Unlock();
	}

	void Clear()
	{
		L::Lock();
		for(int32 i=0; i<NumIn; ++i)
		{
			Elements[IdxOut].~T();
			if (++IdxOut == MaxNum)
			{
				IdxOut -= MaxNum;
			}
		}
		IdxIn = IdxOut = NumIn = 0;
		L::Unlock();
	}

	void Resize(int32 newCapacity)
	{
		L::Lock();
		if (newCapacity == 0)
		{
			Clear();
			M::Deallocate(Elements);
			Elements = nullptr;
			MaxNum = 0;
		}
		else
		{
			// Create a new array and copy all active elements over.
			checkf(newCapacity >= Num(), TEXT("Cannot shrink the dynamic queue to %u with %u elements currently managed!"), (uint32)newCapacity, (uint32)Num());
			T* pNewElements = M::Allocate(newCapacity);
			for(int32 i = 0, j = IdxOut; i < NumIn; ++i)
			{
				new ((void*)(pNewElements + i)) T(Elements[j]);
				Elements[j].~T();
				if (++j == MaxNum)
				{
					j = 0;
				}
			}
			M::Deallocate(Elements);
			Elements = pNewElements;
			IdxIn = NumIn;
			IdxOut = 0;
			MaxNum = newCapacity;
		}
		L::Unlock();
	}

	int32 Increment() const
	{
		return IncrementBy;
	}

	int32 Capacity() const
	{
		return MaxNum;
	}

	int32 Num() const
	{
		return NumIn;
	}

	bool IsEmpty() const
	{
		return Num() == 0;
	}

	bool IsFull() const
	{
		return Num() == Capacity();
	}

	void Push(const T& element)
	{
		L::Lock();
		if (IsFull())
		{
			Resize(Capacity() + Increment());
		}
		int32 pos = IdxIn;
		if (++IdxIn == MaxNum)
		{
			IdxIn = 0;
		}
		++NumIn;
		// Copying the element can NOT take place outside the lock in case multiple threads push at the same time and a Resize() will happen.
		new ((void*)(Elements + pos)) T(element);
		L::Unlock();
	}

	void PushFront(const T& element)
	{
		L::Lock();
		if (IsFull())
		{
			Resize(Capacity() + Increment());
		}
		if (IdxOut == 0)
		{
			IdxOut = MaxNum - 1;
		}
		else
		{
			--IdxOut;
		}
		int32 pos = IdxOut;
		++NumIn;
		// Copying the element can NOT take place outside the lock in case multiple threads push at the same time and a Resize() will happen.
		new ((void*)(Elements + pos)) T(element);
		L::Unlock();
	}

	T Pop()
	{
		L::Lock();
		check(!IsEmpty());
		T r(Elements[IdxOut]);
		Elements[IdxOut].~T();
		if (++IdxOut == MaxNum)
		{
			IdxOut = 0;
		}
		--NumIn;
		L::Unlock();
		return r;
	}

	T Front() const
	{
		L::Lock();
		check(!IsEmpty());
		T r(Elements[IdxOut]);
		L::Unlock();
		return r;
	}

protected:
	T* Elements = nullptr;
	int32 MaxNum = 0;
	int32 NumIn = 0;
	int32 IdxIn = 0;
	int32 IdxOut = 0;
	int32 IncrementBy = 32;
};





template <typename T, typename M = TMediaQueueAllocator<T> >
class TMediaQueueDynamicNoLock : public TMediaQueueDynamic<T, FMediaLockNone, M>
{
public:
	using MyType = TMediaQueueDynamicNoLock<T, M>;
	using ElementType = T;
	typedef TMediaQueueDynamic<T, FMediaLockNone, M>	B;

	TMediaQueueDynamicNoLock() : B() {}
	~TMediaQueueDynamicNoLock() = default;

	const T& FrontRef() const
	{
		check(!B::IsEmpty());
		return B::Elements[B::IdxOut];
	}

	T& FrontRef()
	{
		check(!B::IsEmpty());
		return B::Elements[B::IdxOut];
	}

	const T& BackRef() const
	{
		check(!B::IsEmpty());
		return B::Elements[(B::IdxIn + B::Capacity() - 1) % B::Capacity()];
	}

	T& BackRef()
	{
		check(!B::IsEmpty());
		return B::Elements[(B::IdxIn + B::Capacity() - 1) % B::Capacity()];
	}

	const T& operator[](int32 i) const
	{
		check(!B::IsEmpty());
		check(i < B::NumIn);
		i += B::IdxOut;
		if (i >= B::MaxNum)
		{
			i -= B::MaxNum;
		}
		return B::Elements[i];
	}

	T& operator[](int32 i)
	{
		check(!B::IsEmpty());
		check(i < B::NumIn);
		i += B::IdxOut;
		if (i >= B::MaxNum)
		{
			i -= B::MaxNum;
		}
		return B::Elements[i];
	}
};
