// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "Misc/ScopeLock.h"
#include "Misc/DateTime.h"
#include "Core/MediaNoncopyable.h"


class FMediaSemaphore : private TMediaNoncopyable<FMediaSemaphore>
{
public:
	FMediaSemaphore(int32 InitialCount = 0)
		: Event(FPlatformProcess::GetSynchEventFromPool(true))
		, Count(InitialCount)
	{ }

	~FMediaSemaphore()
	{
		CriticalSection.Lock();
		Event->Trigger();
		FPlatformProcess::ReturnSynchEventToPool(Event);
		Event = nullptr;
		CriticalSection.Unlock();
		Count = 0;
	}

	bool Obtain()
	{
		CriticalSection.Lock();

		// Check if decrement operation would work. We inspect the value before the decrment operation
		// INFO: The UE implementation of InterlockedDecrement returns the value after the decrement operation
		while(FPlatformAtomics::InterlockedDecrement(&Count) <= -1)
		{
			// Undo the "wrong" decrement operation to restore value...
			FPlatformAtomics::InterlockedIncrement(&Count);

			// Before we wait, clear signal wait state. This must happen while locked. Otherwise we would clear the Event trigger from an simultaneously running Release()
			Event->Reset();

			// Release the lock and wait. The event could be triggered already, but the Event->Reset() was protected by the lock and we get signaled for any previous trigger
			CriticalSection.Unlock();
			if (!Event->Wait())
			{
				return false;
			}
			CriticalSection.Lock();
		}
		CriticalSection.Unlock();
		return true;
	}

	bool Obtain(int64 WaitMicroSeconds)
	{
		CriticalSection.Lock();

		FTimespan WaitTime = FTimespan::FromMicroseconds(WaitMicroSeconds);
		int64 TicksStart = FDateTime::Now().GetTicks();

		// Check if decrement operation would work.
		while(FPlatformAtomics::InterlockedDecrement(&Count) <= -1)
		{
			// Undo our decrement operations, because we know this was wrong...
			FPlatformAtomics::InterlockedIncrement(&Count);

			// Before we wait, clear signal wait state. This must happen while locked. Otherwise we would clear the Event trigger from an simultaneously running Release()
			Event->Reset();

			// Release the lock and wait. The event could be triggered already, but the Event->Reset() was protected by the lock and we get signaled for any previous trigger
			CriticalSection.Unlock();

			// For some strange reason, no waiting time left anymore... Exit!
			if (WaitTime <= FTimespan::Zero())
			{
				return false;
			}

			// Check if the wait timed out...
			if (!Event->Wait(WaitTime))
			{
				// Time out. Nothing can safe us here...
				return false;
			}

			// Event was triggered.
			CriticalSection.Lock();

			// Calculate remaining waiting time as we could be triggered, BUT still will not get the value, because some other thread grabbed it first!
			int64 TicksUntilHere = FDateTime::Now().GetTicks();
			FTimespan DeltaWaiting(TicksUntilHere - TicksStart);
			// Decrement time we are allowed to wait
			WaitTime -= DeltaWaiting;
			TicksStart = TicksUntilHere;
		}
		CriticalSection.Unlock();
		return true;
	}

	bool TryToObtain()
	{
		FScopeLock Lock(&CriticalSection);
		// Try if decrement operation would work. We inspect the value before the decrment operation...
		if (FPlatformAtomics::InterlockedDecrement(&Count) <= -1)
		{
			// Nope: Undo the "wrong" decrement operation to restore value...
			FPlatformAtomics::InterlockedIncrement(&Count);
			return false;
		}
		return true;
	}

	void Release()
	{
		FScopeLock Lock(&CriticalSection);
		FPlatformAtomics::InterlockedIncrement(&Count);
		Event->Trigger();
	}

	int32 CurrentCount() const
	{
		FScopeLock Lock(&CriticalSection);
		return Count;
	}

	void SetCount(int32 NewCurrentCount)
	{
		FScopeLock Lock(&CriticalSection);
		Count = NewCurrentCount;
	}

private:
	mutable FCriticalSection CriticalSection;
	mutable FEvent* Event;
	int32 Count;
};
