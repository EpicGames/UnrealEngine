// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/MediaTypes.h"
#include "Core/MediaMacros.h"
#include "Core/MediaNoncopyable.h"
#include "Core/MediaEventSignal.h"

#include "HAL/PlatformAffinity.h"
#include "HAL/Runnable.h"
#include "HAL/PlatformProcess.h"
#include "HAL/CriticalSection.h"
#include "Delegates/Delegate.h"

#include "Containers/UnrealString.h"
#include "Containers/StringConv.h"
#include "Templates/SharedPointer.h"
#include "Templates/Function.h"

#define UE_API ELECTRABASE_API


class FMediaRunnable : private TMediaNoncopyable<FMediaRunnable>, public FRunnable
{
public:
	DECLARE_DELEGATE(FStartDelegate);

	/**
	 * Common thread configuration parameters.
	 */
	struct Param
	{
		// Set some standard values as defaults.
		EThreadPriority Priority = TPri_Normal;
		uint32 StackSize = 65536;;
		int32 CoreAffinity = -1;
	};

	static UE_API void Startup();
	static UE_API void Shutdown();
	static UE_API void EnqueueAsyncTask(TFunction<void()>&& InFunctionToExecuteOnAsyncThread);

	static UE_API FMediaRunnable* Create(int32 CoreAffinityMask, EThreadPriority Priority, uint32 StackSize, const FString& InThreadName);
	static UE_API void Destroy(FMediaRunnable* Thread);

	UE_API void Start(FStartDelegate Entry, bool bWaitRunning = false);

	UE_API void SetDoneSignal(FMediaEvent* DoneSignal);

	UE_API void SetName(const FString& InThreadName);

	UE_API EThreadPriority ChangePriority(EThreadPriority NewPriority);

	EThreadPriority PriorityGet() const
	{
		return ThreadPriority;
	}

	uint32 StackSizeGet() const
	{
		return StackSize;
	}

	static uint32 StackSizeGetDefault()
	{
		return 0;
	}

	static void SleepSeconds(uint32 Seconds)
	{
		FPlatformProcess::Sleep((float)Seconds);
	}

	static void SleepMilliseconds(uint32 Milliseconds)
	{
		FPlatformProcess::Sleep(Milliseconds / 1000.0f);
	}

	static void SleepMicroseconds(uint32 Microseconds)
	{
		FPlatformProcess::Sleep(Microseconds / 1000000.0f);
	}

private:
	FCriticalSection StateAccessMutex;
	FStartDelegate EntryFunction;
	FString ThreadName;
	FRunnableThread* MediaThreadRunnable = nullptr;
	FEvent* SignalRunning = nullptr;
	FMediaEvent* DoneSignal = nullptr;
	EThreadPriority ThreadPriority = TPri_Normal;
	uint64 InitialCoreAffinity = 0;
	uint32 StackSize = 0;
	bool bIsStarted = false;

	UE_API FMediaRunnable();
	UE_API ~FMediaRunnable();

	UE_API void StartInternal();
	UE_API uint32 Run() override;
	UE_API void Exit() override;
};


/**
 * A thread base class to either inherit from or use as a variable.
 *
 * Thread parameters are given to constructor but can be changed before starting the thread
 * with the ThreadSet..() functions.
 * This allows the constructor to use system default values so an instance of this class
 * can be used as a member variable of some class.
 *
 * To start the thread on some function of type void(*)(void) call ThreadStart() with
 * an appropriate delegate.
 *
 * IMPORTANT: If used as a member variable you have to ensure for your class to stay alive
 *            i.e. not be destroyed before the thread function has finished. Otherwise your
 *            function will most likely crash.
 *            Either wait for thread completion by calling ThreadWaitDone() *OR*
 *            have the destructor wait for the thread itself by calling ThreadWaitDoneOnDelete(true)
 *            at some point prior to destruction - preferably before starting the thread.
 */
class FMediaThread : private TMediaNoncopyable<FMediaThread>
{
public:
	UE_API virtual ~FMediaThread();

	// Default constructor. Uses system defaults if no values given.
	UE_API FMediaThread(const char* AnsiName = nullptr);

	// Set a thread priority other than the one given to the constructor before starting the thread.
	UE_API void ThreadSetPriority(EThreadPriority Priority);

	// Set a core affinity before starting the thread. Defaults to -1 for no affinity (run on any core).
	UE_API void ThreadSetCoreAffinity(int32 CoreAffinity);

	// Set a thread stack size other than the one given to the constructor before starting the thread.
	UE_API void ThreadSetStackSize(uint32 StackSize);

	// Set a thread name other than the one given to the constructor before starting the thread.
	UE_API void ThreadSetName(const char* InAnsiThreadName);

	// Sets a new thread name once the thread is running.
	UE_API void ThreadRename(const char* InAnsiThreadName);

	// Sets whether or not the destructor needs to wait for the thread to have finished. Defaults to false. Useful when using this as a member variable.
	UE_API void ThreadWaitDoneOnDelete(bool bWait);

	// Waits for the thread to have finished.
	UE_API void ThreadWaitDone();

	// Starts the thread at the given function void(*)(void)
	UE_API void ThreadStart(FMediaRunnable::FStartDelegate EntryFunction);

	// Resets the thread to be started again. Must have waited for thread termination using ThreadWaitDone() first!
	UE_API void ThreadReset();

private:
	FMediaEvent SigDone;
	FString ThreadName;
	FMediaRunnable* MediaRunnable = nullptr;
	EThreadPriority Priority = TPri_Normal;
	uint32 StackSize = 0;
	int32 CoreAffinity = 0;
	bool bIsStarted = false;;
	bool bWaitDoneOnDelete = false;
};

#undef UE_API
