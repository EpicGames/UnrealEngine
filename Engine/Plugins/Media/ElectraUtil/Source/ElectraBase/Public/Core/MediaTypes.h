// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"

// Include low level includes. This will also include HAL/Platform.h for our types
#include "CoreTypes.h"
#include "Templates/SharedPointer.h"

#if PLATFORM_MICROSOFT
	#include "Microsoft/WindowsHWrapper.h"
#endif

typedef uint32  UEMediaError;

#define UEMEDIA_ERROR_OK				(0)
#define UEMEDIA_ERROR_DETAIL			(1)
#define UEMEDIA_ERROR_INTERNAL			(2)
#define UEMEDIA_ERROR_NOT_IMPLEMENTED	(3)
#define UEMEDIA_ERROR_ABORTED			(4)
#define UEMEDIA_ERROR_END_OF_STREAM		(5)
#define UEMEDIA_ERROR_INSUFFICIENT_DATA	(6)
#define UEMEDIA_ERROR_READ_ERROR		(7)
#define UEMEDIA_ERROR_FORMAT_ERROR		(8)
#define UEMEDIA_ERROR_NOT_SUPPORTED		(9)
#define UEMEDIA_ERROR_TRY_AGAIN			(100)
#define UEMEDIA_ERROR_OOM				(0x80000000U)
#define UEMEDIA_ERROR_BAD_ARGUMENTS		(0x80000001U)

namespace Electra
{
	template <typename T>
	using TSharedPtrTS = TSharedPtr<T, ESPMode::ThreadSafe>;

	template <typename T>
	using TWeakPtrTS = TWeakPtr<T, ESPMode::ThreadSafe>;

	template<typename T, typename... ArgTypes>
	TSharedRef<T, ESPMode::ThreadSafe> MakeSharedTS(ArgTypes&&... Args)
	{
		return MakeShared<T, ESPMode::ThreadSafe>(Forward<ArgTypes>(Args)...);
	}
}
