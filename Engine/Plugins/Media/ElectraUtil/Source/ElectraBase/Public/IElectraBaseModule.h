// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Modules/ModuleInterface.h"

/**
 * Interface for the `ElectraBase` module.
 */
class IElectraBaseModule
	: public IModuleInterface
{
public:

	virtual ~IElectraBaseModule() = default;

private:
};
