// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Timespan.h"

#include "ParameterDictionary.h"

struct FDecoderTimeStamp
{
	FDecoderTimeStamp() {}
	FDecoderTimeStamp(FTimespan InTime, int64 InSequenceIndex) : Time(InTime), SequenceIndex(InSequenceIndex) {}

	FTimespan Time;
	int64 SequenceIndex;
};


namespace IDecoderOutputOptionNames
{
static const FName PTS(TEXT("pts"));
static const FName Duration(TEXT("duration"));
static const FName Width(TEXT("width"));
static const FName Height(TEXT("height"));
static const FName Pitch(TEXT("pitch"));
static const FName AspectRatio(TEXT("aspect_ratio"));
static const FName CropLeft(TEXT("crop_left"));
static const FName CropTop(TEXT("crop_top"));
static const FName CropRight(TEXT("crop_right"));
static const FName CropBottom(TEXT("crop_bottom"));
static const FName PixelFormat(TEXT("pixelfmt"));
static const FName PixelEncoding(TEXT("pixelenc"));
static const FName Orientation(TEXT("orientation"));
static const FName BitsPerComponent(TEXT("bits_per"));
static const FName HDRInfo(TEXT("hdr_info"));
static const FName Colorimetry(TEXT("colorimetry"));
static const FName AspectW(TEXT("aspect_w"));
static const FName AspectH(TEXT("aspect_h"));
static const FName FPSNumerator(TEXT("fps_num"));
static const FName FPSDenominator(TEXT("fps_denom"));
static const FName PixelDataScale(TEXT("pix_datascale"));
static const FName Timecode(TEXT("timecode"));
static const FName TMCDTimecode(TEXT("tmcd_timecode"));
static const FName TMCDFramerate(TEXT("tmcd_framerate"));
}
