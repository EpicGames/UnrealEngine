// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <CoreMinimal.h>
#include <Misc/Timespan.h>
#include <Templates/SharedPointer.h>

#include "MediaDecoderOutput.h"

class IMetaDataDecoderOutput
{
public:
	virtual ~IMetaDataDecoderOutput() = default;

	virtual const void* GetData() = 0;
	virtual FTimespan GetDuration() const = 0;
	virtual uint32 GetSize() const = 0;
	virtual FDecoderTimeStamp GetTime() const = 0;

	enum class EOrigin
	{
		EventStream,
		InbandEventStream,
		TimedMetadata
	};

	enum class EDispatchedMode
	{
		OnReceive,
		OnStart
	};

	// Returns where the event originated from.
	virtual EOrigin GetOrigin() const = 0;

	// Returns the type with which this event was dispatched.
	virtual EDispatchedMode GetDispatchedMode() const = 0;

	// Returns the event's scheme id URI
	virtual const FString& GetSchemeIdUri() const = 0;

	// Returns the event's value.
	virtual const FString& GetValue() const = 0;

	// Returns the event's ID.
	virtual const FString& GetID() const = 0;

	// Returns the base time (the "zero point") of the track, if available, to calculate the relative time
	// of this metadata from GetTime() if this is based on some non-zero anchor.
	virtual TOptional<FDecoderTimeStamp> GetTrackBaseTime() const = 0;

	virtual void SetTime(FDecoderTimeStamp& InTime) = 0;
};

using IMetaDataDecoderOutputPtr = TSharedPtr<IMetaDataDecoderOutput, ESPMode::ThreadSafe>;
