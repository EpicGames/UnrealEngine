// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <CoreMinimal.h>
#include <Misc/Timespan.h>
#include <Templates/SharedPointer.h>

#include "MediaDecoderOutput.h"

class ISubtitleDecoderOutput
{
public:
	virtual ~ISubtitleDecoderOutput() = default;

	virtual const TArray<uint8>& GetData() = 0;
	virtual FDecoderTimeStamp GetTime() const = 0;
	virtual FTimespan GetDuration() const = 0;

	virtual const FString& GetFormat() const = 0;
	virtual const FString& GetID() const = 0;

	virtual void SetTime(FDecoderTimeStamp& InTime) = 0;
};

using ISubtitleDecoderOutputPtr = TSharedPtr<ISubtitleDecoderOutput, ESPMode::ThreadSafe>;
