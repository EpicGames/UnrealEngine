// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"

namespace Electra
{
	/**
	 * This struct describes a URL used in media streaming.
	 *
	 * It describes the URL only, along with CDN and priority for content steering,
	 * but it DOES NOT and MUST NOT include anything request related, like a
	 * content byte range, headers, cookies and so on.
	 */
	struct FMediaURL
	{
		/**
		 * The URL itself
		 */
		FString URL;

		/**
		 * The "CDN" this URL is hosted on, if known
		 *
		 * DASH: @serviceLocation attribute
		 * HLS: PATHWAY-ID attribute
		 * other: not used
		 */
		FString CDN;

		/**
		 * DVB DASH prioritization attributes.
		 */
		int32 DVBweight = 1;
		int32 DVBpriority = 1;

		void Empty()
		{
			URL.Empty();
			CDN.Empty();
			DVBweight = 1;
			DVBpriority = 1;
		}
	};
}
