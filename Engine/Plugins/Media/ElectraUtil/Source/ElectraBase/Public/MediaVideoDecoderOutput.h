// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"

// -------------------------------------------------------------------------------------------------------------------------------

class IVideoDecoderTimecode : public TSharedFromThis<IVideoDecoderTimecode, ESPMode::ThreadSafe>
{
public:
	struct FMPEGDefinition
	{
		// Calculated value as per:
		//   clockTimestamp = ( ( hH * 60 + mM ) * 60 + sS ) * time_scale + nFrames * ( num_units_in_tick * ( 1 + nuit_field_based_flag ) ) + tOffset,
		// can only be valid when there is timing information.
		int64 clockTimestamp = 0;
		// Values from a pic_timing() SEI in H.264 or from a time_code() SEI in H.265
		uint32 num_units_in_tick = 0;			// from the SPS
		uint32 time_scale = 0;					// from the SPS
		int32 time_offset = 0;
		uint16 n_frames = 0;
		uint8 timing_info_present_flag = 0;		// from the SPS
		uint8 clock_timestamp_flag = 0;
		uint8 ct_type = 0;
		uint8 nuit_field_based_flag = 0;
		uint8 counting_type = 0;
		uint8 full_timestamp_flag = 0;
		uint8 discontinuity_flag = 0;
		uint8 cnt_dropped_flag = 0;
		uint8 seconds_value = 0;
		uint8 minutes_value = 0;
		uint8 hours_value = 0;
		uint8 FromH26x = 0;		// last digit of the codec this comes from. 4=H.264, 5=H.265, etc. In case values need different interpretation.
	};

	virtual ~IVideoDecoderTimecode() = default;
	virtual FMPEGDefinition const* GetMPEGDefinition() const = 0;
};



class IVideoDecoderColorimetry : public TSharedFromThis<IVideoDecoderColorimetry, ESPMode::ThreadSafe>
{
public:
	struct FMPEGDefinition
	{
		uint8 ColourPrimaries = 2;
		uint8 TransferCharacteristics = 2;
		uint8 MatrixCoefficients = 2;
		uint8 VideoFullRangeFlag = 0;
		uint8 VideoFormat = 5;
	};

	virtual ~IVideoDecoderColorimetry() = default;
	virtual FMPEGDefinition const* GetMPEGDefinition() const = 0;
};



struct FVideoDecoderHDRMetadata_mastering_display_colour_volume
{
	// Index 0=red, 1=green, 2=blue
	float display_primaries_x[3];
	float display_primaries_y[3];
	float white_point_x;
	float white_point_y;
	float max_display_mastering_luminance;
	float min_display_mastering_luminance;
};

struct FVideoDecoderHDRMetadata_content_light_level_info
{
	uint16 max_content_light_level = 0;			// MaxCLL
	uint16 max_pic_average_light_level = 0;		// MaxFALL
};



class IVideoDecoderHDRInformation : public TSharedFromThis<IVideoDecoderHDRInformation, ESPMode::ThreadSafe>
{
public:
	enum class EType
	{
		Unknown,
		PQ10,				// 10 bit HDR, no metadata.
		HDR10,				// 10 bit HDR, static metadata (mastering display colour volume + content light level info)
		HLG10				// 10 bit HDR, static metadata (mastering display colour volume + content light level info) (HLG transfer characteristics)
	};

	virtual ~IVideoDecoderHDRInformation() = default;

	// Returns the type of HDR in use.
	virtual EType GetHDRType() const = 0;

	// Get mastering display colour volume if available. Returns nullptr if information is not available.
	virtual const FVideoDecoderHDRMetadata_mastering_display_colour_volume* GetMasteringDisplayColourVolume() const = 0;

	// Get content light level info if available. Returns nullptr if information is not available.
	virtual const FVideoDecoderHDRMetadata_content_light_level_info* GetContentLightLevelInfo() const = 0;
};
