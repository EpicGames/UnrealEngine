// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/ScopeLock.h"
#include "PlayerTime.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Containers/Array.h"
#include "Misc/Variant.h"
#include "Misc/Timespan.h"
#include "Misc/Timecode.h"
#include "Misc/FrameRate.h"

#define UE_API ELECTRABASE_API


namespace Electra
{

class FVariantValue
{
	class FSharedPtrHolderBase
	{
	public:
		virtual ~FSharedPtrHolderBase() = default;
		virtual void SetValueOn(FSharedPtrHolderBase* Dst) const = 0;
	};

	template<typename T> class TSharedPtrHolder : public FSharedPtrHolderBase
	{
		using SharedPtrType = TSharedPtr<T, ESPMode::ThreadSafe>;

	public:
		TSharedPtrHolder(const SharedPtrType & InPointer)
			: Pointer(InPointer)
		{}

		SharedPtrType	Pointer;

		virtual void SetValueOn(FSharedPtrHolderBase* Dst) const override
		{
			new(Dst) TSharedPtrHolder<T>(reinterpret_cast<const SharedPtrType&>(Pointer));
		}
	};

public:
	UE_API FVariantValue();
	UE_API ~FVariantValue();
	UE_API FVariantValue(const FVariantValue& rhs);
	UE_API FVariantValue& operator=(const FVariantValue& rhs);

	UE_API explicit FVariantValue(const FString& StringValue);
	UE_API explicit FVariantValue(double DoubleValue);
	UE_API explicit FVariantValue(int64 Int64Value);
	UE_API explicit FVariantValue(bool BoolValue);
	UE_API explicit FVariantValue(const FTimeValue& TimeValue);
	UE_API explicit FVariantValue(const FTimespan& TimespanValue);
	UE_API explicit FVariantValue(const FTimecode& TimecodeValue);
	UE_API explicit FVariantValue(const FFrameRate& FramerateValue);
	UE_API explicit FVariantValue(void* PointerValue);
	template<typename T> explicit FVariantValue(const TSharedPtr<T, ESPMode::ThreadSafe>& PointerValue)
	: DataType(EDataType::TypeUninitialized)
	{
		Set(PointerValue);
	}
	UE_API explicit FVariantValue(const TArray<uint8>& ArrayValue);

	UE_API FVariantValue& Set(const FString& StringValue);
	UE_API FVariantValue& Set(double DoubleValue);
	UE_API FVariantValue& Set(int64 Int64Value);
	UE_API FVariantValue& Set(bool BoolValue);
	UE_API FVariantValue& Set(const FTimeValue& TimeValue);
	UE_API FVariantValue& Set(const FTimespan& TimespanValue);
	UE_API FVariantValue& Set(const FTimecode& TimecodeValue);
	UE_API FVariantValue& Set(const FFrameRate& FramerateValue);
	UE_API FVariantValue& Set(void* PointerValue);
	template <typename T> FVariantValue& Set(const TSharedPtr<T, ESPMode::ThreadSafe>& PointerValue)
	{
		Clear();
		new(&DataBuffer) TSharedPtrHolder<T>(PointerValue);
		DataType = EDataType::TypeSharedPointer;
		return *this;
	}
	UE_API FVariantValue& Set(const TArray<uint8>& ArrayValue);

	// Returns variant value. Type *must* match. Otherwise an empty/zero value is returned.
	UE_API const FString& GetFString() const;
	UE_API const double& GetDouble() const;
	UE_API const int64& GetInt64() const;
	UE_API const bool& GetBool() const;
	UE_API const FTimeValue& GetTimeValue() const;
	UE_API const FTimespan& GetTimespan() const;
	UE_API const FTimecode& GetTimecode() const;
	UE_API const FFrameRate& GetFramerate() const;
	UE_API void* const & GetPointer() const;
	template<typename T> TSharedPtr<T, ESPMode::ThreadSafe> GetSharedPointer() const
	{
		if (ensure(DataType == EDataType::TypeSharedPointer))
		{
			const TSharedPtrHolder<T>* Pointer = reinterpret_cast<const TSharedPtrHolder<T>*>(&DataBuffer);
			return Pointer->Pointer;
		}
		else
		{
			return TSharedPtr<T, ESPMode::ThreadSafe>();
		}
	}
	UE_API const TArray<uint8>& GetArray() const;

	// Returns variant value. If type does not match the specified default will be returned.
	UE_API const FString& SafeGetFString(const FString& Default = FString()) const;
	UE_API double SafeGetDouble(double Default=0.0) const;
	UE_API int64 SafeGetInt64(int64 Default=0) const;
	UE_API bool SafeGetBool(bool Default=false) const;
	UE_API FTimeValue SafeGetTimeValue(const FTimeValue& Default=FTimeValue()) const;
	UE_API FTimespan SafeGetTimespan(const FTimespan& Default=FTimespan()) const;
	UE_API FTimecode SafeGetTimecode(const FTimecode& Default=FTimecode()) const;
	UE_API FFrameRate SafeGetFramerate(const FFrameRate& Default=FFrameRate()) const;
	UE_API void* SafeGetPointer(void* Default=nullptr) const;
	UE_API const TArray<uint8>& SafeGetArray() const;

	enum class EDataType
	{
		TypeUninitialized,
		TypeFString,
		TypeDouble,
		TypeInt64,
		TypeBoolean,
		TypeTimeValue,
		TypeTimespanValue,
		TypeTimecodeValue,
		TypeFramerateValue,
		TypeVoidPointer,
		TypeSharedPointer,
		TypeU8Array,
	};
	EDataType GetDataType() const
	{
		return DataType;
	}

	bool IsValid() const
	{
		return DataType != EDataType::TypeUninitialized;
	}

	bool IsType(EDataType type) const
	{
		return DataType == type;
	}

	UE_API FVariant ToFVariant() const;

private:

	union FUnionLayout
	{
		uint8 MemSizeFString[sizeof(FString)];
		uint8 MemSizeDouble[sizeof(double)];
		uint8 MemSizeInt64[sizeof(int64)];
		uint8 MemSizeVoidPtr[sizeof(void*)];
		uint8 MemSizeBoolean[sizeof(bool)];
		uint8 MemSizeTimeValue[sizeof(FTimeValue)];
		uint8 MemSizeTimespan[sizeof(FTimespan)];
		uint8 MemSizeTimecode[sizeof(FTimecode)];
		uint8 MemSizeFramerate[sizeof(FFrameRate)];
		uint8 MemSizeSharedPtrValue[sizeof(TSharedPtrHolder<uint8>)];
		uint8 MemSizeTArray[sizeof(TArray<uint8>)];
	};

	UE_API void Clear();
	UE_API void CopyInternal(const FVariantValue& FromOther);

	TAlignedBytes<sizeof(FUnionLayout), 16> DataBuffer;
	EDataType DataType;
};



class FParamDict
{
public:
	FParamDict() {}
	UE_API FParamDict(const FParamDict& Other);
	UE_API FParamDict& operator=(const FParamDict& Other);
	~FParamDict() = default;
	UE_API void Clear();
	UE_API void Set(const FName& Key, const FVariantValue& Value);
	UE_API void Set(const FName& Key, FVariantValue&& Value);
	UE_API void GetKeys(TArray<FName>& OutKeys) const;
	UE_API bool HaveKey(const FName& Key) const;
	UE_API FVariantValue GetValue(const FName& Key) const;
	UE_API void Remove(const FName& Key);
	UE_API bool SetValueFrom(FName InKey, const FParamDict& InOther);

	UE_API void ConvertTo(TMap<FString, FVariant>& OutVariantMap, const FString& InAddPrefixToKey) const;
	UE_API void ConvertKeysStartingWithTo(TMap<FString, FVariant>& OutVariantMap, const FString& InKeyStartsWith, const FString& InAddPrefixToKey) const;
private:
	UE_API void InternalCopy(const FParamDict& Other);
	TMap<FName, FVariantValue> Dictionary;
};

class FParamDictTS
{
public:
	FParamDictTS() {}
	FParamDictTS(const FParamDictTS& Other)
	{
		Dictionary = Other.Dictionary;
	}
	FParamDictTS& operator=(const FParamDictTS& Other)
	{
		if (&Other != this)
		{
			FScopeLock lock(&Lock);
			Dictionary = Other.Dictionary;
		}
		return *this;
	}
	FParamDictTS& operator=(const FParamDict& Other)
	{
		FScopeLock lock(&Lock);
		Dictionary = Other;
		return *this;
	}

	~FParamDictTS() = default;

	FParamDict GetDictionary() const
	{
		FScopeLock lock(&Lock);
		return Dictionary;
	}

	void Clear()
	{
		FScopeLock lock(&Lock);
		Dictionary.Clear();
	}
	void Set(const FName& Key, const FVariantValue& Value)
	{
		FScopeLock lock(&Lock);
		Dictionary.Set(Key, Value);
	}
	void Set(const FName& Key, FVariantValue&& Value)
	{
		FScopeLock lock(&Lock);
		Dictionary.Set(Key, MoveTemp(Value));
	}
	void GetKeys(TArray<FName>& OutKeys) const
	{
		FScopeLock lock(&Lock);
		Dictionary.GetKeys(OutKeys);
	}
	bool HaveKey(const FName& Key) const
	{
		FScopeLock lock(&Lock);
		return Dictionary.HaveKey(Key);
	}
	FVariantValue GetValue(const FName& Key) const
	{
		FScopeLock lock(&Lock);
		return Dictionary.GetValue(Key);
	}
	void Remove(const FName& Key)
	{
		FScopeLock lock(&Lock);
		Dictionary.Remove(Key);
	}
	bool SetValueFrom(FName InKey, const FParamDict& InOther)
	{
		FScopeLock lock(&Lock);
		return Dictionary.SetValueFrom(InKey, InOther);
	}
	void ConvertTo(TMap<FString, FVariant>& OutVariantMap, const FString& InAddPrefixToKey) const
	{
		FScopeLock lock(&Lock);
		Dictionary.ConvertTo(OutVariantMap, InAddPrefixToKey);
	}
	void ConvertKeysStartingWithTo(TMap<FString, FVariant>& OutVariantMap, const FString& InKeyStartsWith, const FString& InAddPrefixToKey) const
	{
		FScopeLock lock(&Lock);
		Dictionary.ConvertKeysStartingWithTo(OutVariantMap, InKeyStartsWith, InAddPrefixToKey);
	}
private:
	mutable FCriticalSection Lock;
	FParamDict Dictionary;
};


} // namespace Electra

#undef UE_API
