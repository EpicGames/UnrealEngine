// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Timespan.h"
#include "TimeFraction.h"
#include <limits>

namespace Electra
{

/**
 * Keeps a time value in hundred nanoseconds (HNS).
 */
class FTimeValue
{
public:
	static FTimeValue GetInvalid()
	{
		static FTimeValue kInvalid;
		return kInvalid;
	}

	static FTimeValue GetZero()
	{
		static FTimeValue kZero((int64)0);
		return kZero;
	}

	static FTimeValue GetPositiveInfinity()
	{
		static FTimeValue kPosInf(std::numeric_limits<double>::infinity());
		return kPosInf;
	}

	static FTimeValue GetNegativeInfinity()
	{
		static FTimeValue kNegInf(-std::numeric_limits<double>::infinity());
		return kNegInf;
	}

	static int64 MillisecondsToHNS(int64 InMilliseconds)
	{
		return InMilliseconds * 10000;
	}

	static int64 MicrosecondsToHNS(int64 InMicroseconds)
	{
		return InMicroseconds * 10;
	}

	static int64 NinetykHzToHNS(int64 In90kHz)
	{
		return In90kHz * 1000 / 9;
	}

	FTimeValue()
		: HNS(0)
		, SequenceIndex(0)
		, bIsValid(false)
		, bIsInfinity(false)
	{
	}

	FTimeValue(const FTimeValue& rhs)
		: HNS(rhs.HNS)
		, SequenceIndex(rhs.SequenceIndex)
		, bIsValid(rhs.bIsValid)
		, bIsInfinity(rhs.bIsInfinity)
	{
	}

	FTimeValue& operator=(const FTimeValue& rhs)
	{
		HNS = rhs.HNS;
		SequenceIndex = rhs.SequenceIndex;
		bIsValid = rhs.bIsValid;
		bIsInfinity = rhs.bIsInfinity;
		return *this;
	}

	explicit FTimeValue(int64 InHNS, int64 InSequenceIndex=0) : HNS(InHNS), SequenceIndex(InSequenceIndex), bIsValid(true), bIsInfinity(false)
	{
	}
	explicit FTimeValue(double Seconds, int64 InSequenceIndex=0)
	{
		SetFromSeconds(Seconds, InSequenceIndex);
	}
	explicit FTimeValue(int64 Numerator, uint32 Denominator, int64 InSequenceIndex=0)
	{
		SetFromND(Numerator, Denominator, InSequenceIndex);
	}

	bool IsValid() const
	{
		return bIsValid;
	}

	bool IsZero() const
	{
		return bIsValid && HNS == 0;
	}

	bool IsInfinity() const
	{
		return bIsInfinity;
	}

	bool IsPositiveInfinity() const
	{
		return bIsInfinity && HNS >= 0;
	}

	bool IsNegativeInfinity() const
	{
		return bIsInfinity && HNS < 0;
	}


	double GetAsSeconds(double DefaultIfInvalid=0.0) const
	{
		return bIsValid ? (bIsInfinity ? (HNS >= 0 ? std::numeric_limits<double>::infinity() : -std::numeric_limits<double>::infinity()) : HNS / 10000000.0) : DefaultIfInvalid;
	}

	int64 GetAsMilliseconds(int64 DefaultIfInvalid=0) const
	{
		return bIsValid ? (bIsInfinity ? (HNS >= 0 ? 0x7fffffffffffffffLL : -0x7fffffffffffffffLL) : HNS / 10000) : DefaultIfInvalid;
	}

	int64 GetAsMicroseconds(int64 DefaultIfInvalid=0) const
	{
		return bIsValid ? (bIsInfinity ? (HNS >= 0 ? 0x7fffffffffffffffLL : -0x7fffffffffffffffLL) : HNS / 10) : DefaultIfInvalid;
	}

	int64 GetAsHNS(int64 DefaultIfInvalid=0) const
	{
		return bIsValid ? (bIsInfinity ? (HNS >= 0 ? 0x7fffffffffffffffLL : -0x7fffffffffffffffLL) : HNS) : DefaultIfInvalid;
	}

	int64 GetAs90kHz(int64 DefaultIfInvalid=0) const
	{
		return bIsValid ? (bIsInfinity ? (HNS >= 0 ? 0x7fffffffffffffffLL : -0x7fffffffffffffffLL) : HNS * 9 / 1000) : DefaultIfInvalid;
	}

	//! Returns this time value in a custom timebase. Requires internal bigint conversion and is therefor SLOW!
	ELECTRABASE_API int64 GetAsTimebase(uint32 CustomTimebase) const;

	FTimespan GetAsTimespan() const
	{
		if (!bIsValid)
		{
			return FTimespan::MinValue();
		}
		if (IsInfinity())
		{
			return HNS >= 0 ? FTimespan::MaxValue() : FTimespan::MinValue();
		}
		return FTimespan(HNS);
	}

	FTimeValue& SetToInvalid()
	{
		HNS = 0;
		SequenceIndex = 0;
		bIsValid = false;
		bIsInfinity = false;
		return *this;
	}

	FTimeValue& SetToZero(int64 InSequenceIndex=0)
	{
		HNS = 0;
		SequenceIndex = InSequenceIndex;
		bIsValid = true;
		bIsInfinity = false;
		return *this;
	}

	FTimeValue& SetToPositiveInfinity(int64 InSequenceIndex=0)
	{
		HNS = 0x7fffffffffffffffLL;
		SequenceIndex = InSequenceIndex;
		bIsValid = true;
		bIsInfinity = true;
		return *this;
	}

	FTimeValue& SetToNegativeInfinity(int64 InSequenceIndex=0)
	{
		HNS = -0x7fffffffffffffffLL;
		SequenceIndex = InSequenceIndex;
		bIsValid = true;
		bIsInfinity = true;
		return *this;
	}

	FTimeValue& SetFromSeconds(double Seconds, int64 InSequenceIndex=0)
	{
		if ((bIsInfinity = (Seconds == std::numeric_limits<double>::infinity() || Seconds == -std::numeric_limits<double>::infinity())) == true)
		{
			HNS = Seconds < 0.0 ? -0x7fffffffffffffffLL : 0x7fffffffffffffffLL;
			bIsValid = true;
		}
		else
		{
			if ((bIsValid = Seconds >= -922337203685.0 && Seconds <= 922337203685.0) == true)
			{
				HNS = (int64)(Seconds * 10000000.0);
			}
			else
			{
				check(!"Value cannot be represented!");
				HNS = 0;
			}
		}
		SequenceIndex = InSequenceIndex;
		return *this;
	}


	FTimeValue& SetFromMilliseconds(int64 Milliseconds, int64 InSequenceIndex=0)
	{
		bIsInfinity = false;
		if ((bIsValid = Milliseconds >= -922337203685477 && Milliseconds <= 922337203685477) == true)
		{
			HNS = Milliseconds * 10000;
		}
		else
		{
			check(!"Value cannot be represented!");
			HNS = 0;
		}
		SequenceIndex = InSequenceIndex;
		return *this;
	}

	FTimeValue& SetFromMicroseconds(int64 Microseconds, int64 InSequenceIndex=0)
	{
		bIsInfinity = false;
		if ((bIsValid = Microseconds >= -922337203685477580 && Microseconds <= 922337203685477580) == true)
		{
			HNS = Microseconds * 10;
		}
		else
		{
			check(!"Value cannot be represented!");
			HNS = 0;
		}
		SequenceIndex = InSequenceIndex;
		return *this;
	}

	FTimeValue& SetFromHNS(int64 InHNS, int64 InSequenceIndex=0)
	{
		HNS = InHNS;
		bIsValid = true;
		bIsInfinity = false;
		SequenceIndex = InSequenceIndex;
		return *this;
	}

	FTimeValue& SetFrom90kHz(int64 Ticks, int64 InSequenceIndex=0)
	{
		HNS = Ticks * 1000 / 9;
		bIsValid = true;
		bIsInfinity = false;
		SequenceIndex = InSequenceIndex;
		return *this;
	}

	ELECTRABASE_API FTimeValue& SetFromND(int64 Numerator, uint32 Denominator, int64 InSequenceIndex=0);

	ELECTRABASE_API FTimeValue& SetFromTimeFraction(const FTimeFraction& TimeFraction, int64 InSequenceIndex=0);


	FTimeValue& SetFromTimespan(const FTimespan& InTimespan, int64 InSequenceIndex=0)
	{
		SequenceIndex = InSequenceIndex;
		bIsValid = true;
		if (InTimespan == FTimespan::MaxValue())
		{
			HNS = 0x7fffffffffffffffLL;
			bIsInfinity = true;
		}
		else
		{
			HNS = InTimespan.GetTicks();
			bIsInfinity = false;
		}
		return *this;
	}

	void SetSequenceIndex(int64 InSequenceIndex)
	{
		SequenceIndex = InSequenceIndex;
	}

	int64 GetSequenceIndex() const
	{
		return SequenceIndex;
	}

	/*
		Note: We MUST NOT compare the SequenceIndex in any of the relational operators!
		      It is considered to be a kind of "user value".

			  It is also not really possible to perform calculations on time values of different
			  sequence indices.
			  What should the result be for eg. (HNS=1234,SequenceIndex=4) + (HNS=987,SequenceIndex=5) ?
	*/

	bool operator == (const FTimeValue& rhs) const
	{
		return (!bIsValid && !rhs.bIsValid) || (bIsValid == rhs.bIsValid && bIsInfinity == rhs.bIsInfinity && HNS == rhs.HNS);
	}

	bool operator != (const FTimeValue& rhs) const
	{
		return !(*this == rhs);
	}

	bool operator < (const FTimeValue& rhs) const
	{
		if (bIsValid && rhs.bIsValid)
		{
			if (!bIsInfinity)
			{
				return !rhs.bIsInfinity ? HNS < rhs.HNS : rhs.HNS > 0;
			}
			else
			{
				return rhs.bIsInfinity ? HNS < rhs.HNS : HNS < 0;
			}
		}
		return false;
	}

	bool operator <= (const FTimeValue& rhs) const
	{
		return (!bIsValid || !rhs.bIsValid) ? false : (*this < rhs || *this == rhs);
	}

	bool operator > (const FTimeValue &rhs) const
	{
		return (!bIsValid || !rhs.bIsValid) ? false : !(*this <= rhs);
	}

	bool operator >= (const FTimeValue &rhs) const
	{
		return (!bIsValid || !rhs.bIsValid) ? false : !(*this < rhs);
	}

	inline FTimeValue& operator += (const FTimeValue& rhs)
	{
		if (bIsValid)
		{
			if (rhs.bIsValid)
			{
				if (!bIsInfinity && !rhs.bIsInfinity)
				{
					if (WillOverflow(HNS, HNS, rhs.HNS))
					{
						SetToPositiveInfinity();
					}
				}
				else
				{
					SetToPositiveInfinity();
				}
			}
			else
			{
				SetToInvalid();
			}
		}
		return *this;
	}

	inline FTimeValue& operator -= (const FTimeValue& rhs)
	{
		if (bIsValid)
		{
			if (rhs.bIsValid)
			{
				if (!bIsInfinity && !rhs.bIsInfinity)
				{
					HNS -= rhs.HNS;
				}
				else
				{
					SetToPositiveInfinity();
				}
			}
			else
			{
				SetToInvalid();
			}
		}
		return *this;
	}

	inline FTimeValue& operator /= (int32 Scale)
	{
		if (bIsValid && !bIsInfinity)
		{
			if (Scale)
			{
				HNS /= Scale;
			}
			else
			{
				SetToPositiveInfinity();
			}
		}
		return *this;
	}

	inline FTimeValue& operator *= (int32 Scale)
	{
		if (bIsValid && !bIsInfinity)
		{
		// FIXME: what if this overflows?
			HNS *= Scale;
		}
		return *this;
	}

	inline FTimeValue operator + (const FTimeValue& rhs) const
	{
		FTimeValue Result;
		if (bIsValid && rhs.bIsValid)
		{
			if (!bIsInfinity && !rhs.bIsInfinity)
			{
				Result.bIsValid = true;
				if (WillOverflow(Result.HNS, HNS, rhs.HNS))
				{
					Result.SetToPositiveInfinity();
				}
			}
			else
			{
				Result.SetToPositiveInfinity();
			}
		}
		return Result;
	}

	inline FTimeValue operator - (const FTimeValue& rhs) const
	{
		FTimeValue Result;
		if (bIsValid && rhs.bIsValid)
		{
			if (!bIsInfinity && !rhs.bIsInfinity)
			{
				Result.HNS = HNS - rhs.HNS;
				Result.bIsValid = true;
			}
			else
			{
				Result.SetToPositiveInfinity();
			}
		}
		return Result;
	}

	inline FTimeValue operator << (int32 Shift) const
	{
		FTimeValue Result(*this);
		if (bIsValid)
		{
			if (!bIsInfinity)
			{
				Result.HNS <<= Shift;
			}
			else
			{
				Result.SetToPositiveInfinity();
			}
		}
		return Result;
	}

	inline FTimeValue operator >> (int32 Shift) const
	{
		FTimeValue Result(*this);
		if (bIsValid)
		{
			if (!bIsInfinity)
			{
				Result.HNS >>= Shift;
			}
			else
			{
				Result.SetToPositiveInfinity();
			}
		}
		return Result;
	}

	inline FTimeValue operator * (int32 Scale) const
	{
		FTimeValue Result(*this);
		if (bIsValid)
		{
			if (!bIsInfinity)
			{
				Result.HNS *= Scale;
			}
			else
			{
				Result.SetToPositiveInfinity();
			}
		}
		return Result;
	}

	inline FTimeValue operator / (int32 Scale) const
	{
		FTimeValue Result(*this);
		if (Scale)
		{
			Result.HNS /= Scale;
		}
		else
		{
			Result.SetToPositiveInfinity();
		}
		return Result;
	}

	inline FTimeValue operator - () const
	{
		FTimeValue Result(*this);
		Result.HNS = -Result.HNS;
		return Result;
	}

	inline FTimeValue Abs() const
	{
		FTimeValue Result(*this);
		if (Result.HNS < 0)
		{
			Result.HNS = -Result.HNS;
		}
		return Result;
	}

private:
	static inline bool WillOverflow(int64& OutTemp, const int64 InA, const int64 InB)
	{
		int64 Temp = InA + InB;
		OutTemp = Temp;
		return (InA >= 0 && InB >= 0 && Temp < 0) || (InA < 0 && InB < 0 && Temp >= 0);
	}
	int64	HNS;
	int64	SequenceIndex;
	bool	bIsValid;
	bool	bIsInfinity;
};



struct FTimeRange
{
	void Reset()
	{
		Start.SetToInvalid();
		End.SetToInvalid();
	}
	bool IsValid() const
	{
		return Start.IsValid() && End.IsValid();
	}

	bool Contains(const FTimeValue& Value)
	{
		return Value >= Start && (!End.IsValid() || Value < End);
	}

	bool Overlaps(const FTimeRange& OtherRange)
	{
		return End > OtherRange.Start && Start < OtherRange.End;
	}

	FTimeValue	Start;
	FTimeValue	End;
};

}

