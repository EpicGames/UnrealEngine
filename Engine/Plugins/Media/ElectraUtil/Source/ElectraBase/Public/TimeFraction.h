// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Timespan.h"
#include <limits>

namespace Electra
{

/**
 * Keeps a time value as a fractional.
 */
class FTimeFraction
{
public:
	static const FTimeFraction& GetInvalid()
	{
		static FTimeFraction Invalid;
		return Invalid;
	}

	static const FTimeFraction& GetZero()
	{
		static FTimeFraction Zero(0, 1);
		return Zero;
	}

	static const FTimeFraction& GetPositiveInfinity()
	{
		static FTimeFraction Inf(0, 0);
		return Inf;
	}

	FTimeFraction() = default;

	FTimeFraction(int64 n, uint32 d) : Numerator(n), Denominator(d), bIsValid(true)
	{ }

	FTimeFraction(const FTimeFraction& rhs) : Numerator(rhs.Numerator), Denominator(rhs.Denominator), bIsValid(rhs.bIsValid)
	{ }

	FTimeFraction(const FTimespan& tv)
	{
		SetFromTimespan(tv);
	}

	FTimeFraction& operator=(const FTimeFraction& rhs)
	{
		Numerator = rhs.Numerator;
		Denominator = rhs.Denominator;
		bIsValid = rhs.bIsValid;
		return *this;
	}

	bool IsValid() const
	{
		return bIsValid;
	}

	bool IsPositiveInfinity() const
	{
		return bIsValid && Denominator==0 && Numerator>=0;
	}

	int64 GetNumerator() const
	{
		return Numerator;
	}

	uint32 GetDenominator() const
	{
		return Denominator;
	}

	double GetAsDouble() const
	{
		return Numerator / (double)Denominator;
	}

	/** Returns this time value in a custom timebase. Requires internal bigint conversion and is therefor slow! */
	ELECTRABASE_API int64 GetAsTimebase(uint32 CustomTimebase) const;

	/** Returns this time value as a FTimespan. Requires internal bigint conversion and is therefor slow! */
	ELECTRABASE_API FTimespan GetAsTimespan() const;

	FTimeFraction& SetFromND(int64 InNumerator, uint32 InDenominator)
	{
		Numerator = InNumerator;
		Denominator = InDenominator;
		bIsValid = true;
		return *this;
	}

	FTimeFraction& SetNumerator(int64 InNumerator)
	{
		Numerator = InNumerator;
		bIsValid = true;
		return *this;
	}

	FTimeFraction& SetDenominator(uint32 InDenominator)
	{
		Denominator = InDenominator;
		bIsValid = true;
		return *this;
	}

	FTimeFraction& SetToPositiveInfinity()
	{
		Numerator = 0;
		Denominator = 0;
		bIsValid = true;
		return *this;
	}

	FTimeFraction& SetFromTimespan(const FTimespan& tv)
	{
		Numerator = tv.GetTicks();
		Denominator = (Numerator == ETimespan::MinTicks || Numerator == ETimespan::MaxTicks) ? 0U : (uint32)ETimespan::TicksPerSecond;
		bIsValid = true;
		return *this;
	}

	ELECTRABASE_API FTimeFraction& SetFromFloatString(const FString& In);


	bool operator == (const FTimeFraction& rhs) const
	{
		return bIsValid == rhs.bIsValid && Numerator == rhs.Numerator && Denominator == rhs.Denominator;
	}

	bool operator != (const FTimeFraction& rhs) const
	{
		return !(*this == rhs);
	}

private:
	int64 Numerator = 0;
	uint32 Denominator = 0;
	bool bIsValid = false;
};


}

