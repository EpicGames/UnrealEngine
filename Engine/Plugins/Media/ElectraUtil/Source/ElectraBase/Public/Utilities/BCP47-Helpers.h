// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"

namespace Electra
{

	namespace BCP47
	{

		/**
		 * A parsed RFC-5646 language tag.
		 */
		struct FLanguageTag
		{
			// The full language string if it consists of more than just the primary language.
			FString FullLanguage;
			// The primary language only.
			FString PrimaryLanguage;
			// Any extended language sub parts (up to three) separated by hyphens.
			FString ExtendedLanguage;
			// The script sub part, if given.
			FString Script;
			// The region sub part, if given.
			FString Region;
			// The variant sub parts, if given.
			TArray<FString> Variants;
			// The extension sub parts, if given.
			TArray<FString> Extensions;
			// The private use part, if given.
			FString PrivateUse;

			void Empty()
			{
				FullLanguage.Empty();
				PrimaryLanguage.Empty();
				ExtendedLanguage.Empty();
				Script.Empty();
				Region.Empty();
				Variants.Empty();
				Extensions.Empty();
				PrivateUse.Empty();
			}

			FString Get(bool bAddExtended, bool bAddScript, bool bAddRegion, bool bAddVariants, bool bAddExtensions, bool bAddPrivateUse) const
			{
				FString r(PrimaryLanguage);
				if (bAddExtended && !ExtendedLanguage.IsEmpty())
				{
					r.AppendChar(TCHAR('-'));
					r.Append(ExtendedLanguage);
				}
				if (bAddScript && !Script.IsEmpty())
				{
					r.AppendChar(TCHAR('-'));
					r.Append(Script);
				}
				if (bAddRegion && !Region.IsEmpty())
				{
					r.AppendChar(TCHAR('-'));
					r.Append(Region);
				}
				if (bAddVariants && !Variants.IsEmpty())
				{
					for(auto& v : Variants)
					{
						r.AppendChar(TCHAR('-'));
						r.Append(v);
					}
				}
				if (bAddExtensions && !Extensions.IsEmpty())
				{
					for(auto& e : Extensions)
					{
						r.AppendChar(TCHAR('-'));
						r.Append(e);
					}
				}
				if (bAddPrivateUse && !PrivateUse.IsEmpty())
				{
					r.AppendChar(TCHAR('-'));
					r.Append(PrivateUse);
				}
				return r;
			}

			FString Get() const
			{
				return Get(true, true, true, true, true, true);
			}
		};

		/*
			Parse a single RFC-5646 language tag into its components.
			This will change the language code to the shorter 2 letter ISO-639-1 code if possible
			and change the capitalization of the elements to their recommended case.
			It does NOT canonicalize the tag.
		*/
		ELECTRABASE_API bool ParseRFC5646Tag(FLanguageTag& OutTag, const FString& InRFC5646);


		/*
			Checks an RFC-4647 language priority list for a match against the given language tags
			through RFC-4647 section 3.3.2 "extended filtering".
			Returns the indices into the given language tags that match the language priority list.

			If this filtering produces no match, "lookup" is performed (see RFC-4647 section 3.4)
			with the following differences:
			  - only language ranges not containing a '*' wildcard are considered because the language
			    range must be parseable as a language tag for element-wise comparison.
			  - instead of progressibely truncating the language range from the end the search
			    is performed forward and stops on the first mismatch.
			  - variants, extensions and private use tags are NOT considered!
			  - there is no "default" value to fall back on. The result list will be empty if
			    lookup fails to produce a match.

			The point of performing an additional "lookup" step is to return a better-than-nothing
			result if the language range is over-specified with regard to the given language tags.
			For example, if the language tag is "es" a filter search with only "es-419" will not return
			a match. The search should be "es-419,es-*,es" but constructing this is sometimes difficult.
			This is why a lookup is performed because searching "es-419" will match "es".


			May be an empty list if there are no matches.
		*/
		ELECTRABASE_API TArray<int32> FindExtendedFilteringMatch(const TArray<FLanguageTag>& InTagsToCheck, const FString& InRFC4647Ranges);
	}

}
