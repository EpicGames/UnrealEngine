// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"

#if defined(_MSC_VER)
#include <intrin.h>
#endif

namespace Electra
{

class FBitstreamReader
{
public:
	FBitstreamReader() = default;

	FBitstreamReader(const void* InData, uint64 InDataSize, uint64 StartBytePosition = 0, uint32 StartBitPosition = 0)
	{
		SetData(InData, InDataSize, StartBytePosition, StartBitPosition);
	}

	FBitstreamReader(const FBitstreamReader& rhs)
	{
		Data = rhs.Data;
		DataSize = rhs.DataSize;
		BytePosition = rhs.BytePosition;
		BitPosition = rhs.BitPosition;
	}

	FBitstreamReader& operator = (const FBitstreamReader& rhs)
	{
		Data = rhs.Data;
		DataSize = rhs.DataSize;
		BytePosition = rhs.BytePosition;
		BitPosition = rhs.BitPosition;
		return *this;
	}

	void SetData(const void* InData, uint64 InDataSize, uint64 StartBytePosition = 0, uint32 StartBitPosition = 0)
	{
		check(InData && InDataSize);
		Data = (const uint8*)InData;
		DataSize = InDataSize;
		BytePosition = StartBytePosition;
		BitPosition = StartBitPosition;
	}

	bool IsByteAligned() const
	{
		return (BitPosition & 7) == 0;
	}

	const void* GetRemainingData() const
	{
		return Data + BytePosition;
	}

	uint64 GetRemainingByteLength() const
	{
		return DataSize - BytePosition;
	}

	uint32 GetBytePosition() const
	{
		return BytePosition;
	}

	uint32 GetBitPosition() const
	{
		return BitPosition;
	}

	uint64 GetRemainingBits() const
	{
		return GetRemainingByteLength() * 8 - BitPosition;
	}

	bool GetAlignedBytes(uint8* To, uint64 nBytes)
	{
		check(IsByteAligned());
		if (IsByteAligned() && GetRemainingByteLength() >= nBytes)
		{
			FMemory::Memcpy(To, GetRemainingData(), nBytes);
			BytePosition += nBytes;
			return true;
		}
		return false;
	}

	void SkipBytes(uint64 nBytes)
	{
		BytePosition += nBytes;
		if (BytePosition > DataSize)
		{
			BytePosition = DataSize;
		}
	}

	void SkipBits(uint64 nBits)
	{
		SkipBytes(nBits >> 3);
		GetBits(nBits & 7);
	}

	uint32 GetBits(uint64 nBits)
	{
		if (nBits == 0)
		{
			return 0;
		}
		else if (nBits > GetRemainingBits())
		{
			BytePosition = DataSize;
			BitPosition = 0;
			return 0;
		}
		else
		{
			uint32 rv = PeekBits(nBits);
			uint32 bp = BitPosition + nBits;
			BytePosition += bp >> 3;
			BitPosition = bp & 7;
			return rv;
		}
	}

	uint32 PeekBits(uint64 nBits)
	{
		checkf(nBits <= 32, TEXT("This function can return at most 32 bits!"));
		if (nBits == 0)
		{
			return 0;
		}
		else if (nBits > GetRemainingBits())
		{
			return 0;
		}
		else
		{
			uint32 bytePos = BytePosition + (BitPosition >> 3);
			uint32 bitPos = BitPosition & 7;

			uint32 w;
			uint64 rl = GetRemainingByteLength();
			if (rl >= 4)
			{
#if defined(_MSC_VER)
				w = _byteswap_ulong(*(const uint32*)(Data + bytePos));
#else
				w = __builtin_bswap32(*(const uint32*)(Data + bytePos));
#endif
			}
			else
			{
				if (rl == 3)
				{
					w = ((uint32)Data[bytePos + 0] << 24) | ((uint32)Data[bytePos + 1] << 16) | ((uint32)Data[bytePos + 2] << 8);
				}
				else if (rl == 2)
				{
					w = ((uint32)Data[bytePos + 0] << 24) | ((uint32)Data[bytePos + 1] << 16);
				}
				else if (rl == 1)
				{
					w = ((uint32)Data[bytePos + 0] << 24);
				}
				else
				{
					w = 0;
				}

			}

			if (bitPos == 0)
			{
				w = w >> (32 - nBits);
				return w;
			}
			else
			{
				uint32 w0 = w;
				uint8 w1;
				if (rl > 4)
				{
					w1 = *(const uint8*)(Data + bytePos + 4);
				}
				else
				{
					w1 = 0;
				}
				w = (w0 << bitPos) | uint32(w1 >> (8 - bitPos));
				w = w >> (32 - nBits);
				return w;
			}
		}
	}

	uint64 GetBits64(uint64 nBits)
	{
		if (nBits <= 32)
		{
			return GetBits(nBits);
		}
		else
		{
			uint32 upper = GetBits(nBits - 32);
			uint32 lower = GetBits(32);
			return (uint64(upper) << 32ULL) | uint64(lower);
		}
	}

protected:
	const uint8*	Data = nullptr;
	uint64			DataSize = 0;
	uint64			BytePosition = 0;
	uint32			BitPosition = 0;
};





struct FElectraByteReader
{
	FElectraByteReader(const void* InData, int64 DataSize)
		: Data((const uint8*)InData)
		, NumBytesToGo(DataSize)
	{ }

	bool ReadByte(uint8& To)
	{
		if (NumBytesToGo)
		{
			To = *Data++;
			--NumBytesToGo;
			return true;
		}
		return false;
	}

	bool ReadByte(uint16& To)
	{
		if (NumBytesToGo >= sizeof(uint16))
		{
			uint16 hi = *Data++;
			uint16 lo = *Data++;
			NumBytesToGo -= sizeof(uint16);
			To = (hi << 8) | lo;
			return true;
		}
		return false;
	}

	bool ReadBytes(uint8* To, int64 num)
	{
		if (NumBytesToGo >= num)
		{
			FMemory::Memcpy(To, Data, num);
			Data += num;
			NumBytesToGo -= num;
			return true;
		}
		return false;
	}

	int64 BytesRemaining() const
	{
		return NumBytesToGo;
	}

private:
	const uint8* Data = nullptr;
	int64 NumBytesToGo = 0;
};


class FBitstreamWriter
{
public:
	FBitstreamWriter()
	{
		Buffer.SetNumZeroed(256);
	}
	void PutBits(uint32 InValue, uint32 InNumBits)
	{
		check(InNumBits != 0);
		check(InNumBits <= 32);
		uint32 Mask = 1U << (InNumBits-1);
		while(Mask)
		{
			PutBit((InValue & Mask) != 0 ? 1 : 0);
			Mask >>= 1;
		}
	}
	void PutBits64(uint64 InValue, uint32 InNumBits)
	{
		uint32 Upper = static_cast<uint32>(InValue >> 32U);
		uint32 Lower = static_cast<uint32>(InValue);
		if (InNumBits > 32)
		{
			const uint32 nb = InNumBits - 32;
			PutBits(Upper, nb);
			InNumBits -= nb;
		}
		PutBits(Lower, InNumBits);
	}
	void AlignToBytes(uint32 InFillBitsWith)
	{
		if (BitPosition)
		{
			do
			{
				PutBit(InFillBitsWith);
			} while(BitPosition);
		}
	}
	uint32 GetNumBits() const
	{
		return BytePosition * 8 + BitPosition;
	}
	uint32 GetNumBytesUsed() const
	{
		return (GetNumBits() + 7) / 8;
	}
	bool IsByteAligned() const
	{
		return BitPosition == 0;
	}
	void GetArray(TArray<uint8>& OutArray)
	{
		Buffer.SetNum(GetNumBytesUsed());
		OutArray = MoveTemp(Buffer);
		BytePosition = 0;
		BitPosition = 0;
	}
private:
	void PutBit(uint32 InBit)
	{
		InBit <<= (7 - BitPosition);
		check(Buffer.Num());
		Buffer[BytePosition] |= (uint8)InBit;
		if (++BitPosition == 8)
		{
			BitPosition = 0;
			if (++BytePosition > (uint32)Buffer.Num())
			{
				Buffer.SetNumZeroed(Buffer.Num() + 256);
			}
		}
	}
	TArray<uint8> Buffer;
	uint32 BytePosition = 0;
	uint32 BitPosition = 0;
};

} // namespace Electra
