// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"


namespace Electra
{
	/**
	 * Abstract base class to pass into
	 */
	class IBaseDataReader
	{
	public:
		virtual ~IBaseDataReader() = default;

		enum EResult : int64
		{
			ReadError = -1,
			ReachedEOF = -2,
			Canceled = -3
		};

		/**
		 * A delegate passed into the read methods through which the reader implementation
		 * can call into to see if the read request has been canceled.
		 */
		DECLARE_DELEGATE_RetVal(bool, FCancellationCheckDelegate);

		/**
		 * This method reads data into the provided buffer from the specified absolute offset.
		 * The number of bytes to read should be retrieved unless the end of the file is
		 * reached where returning fewer bytes than requested is permitted.
		 * Reading more bytes than requested is forbidden since the read buffer may not be
		 * large enough to accommodate more bytes than asked for.
		 * Negative return values to indicate a problem are defined in `EResult`
		 * `EResult::Canceled` may be returned even if the provided cancellation delegate
		 * does not indicate cancelation, but the reader implementation has been canceled
		 * by other means (like application shutdown, or in case of an implementation reading
		 * from the network some other condition).
		 */
		virtual int64 ReadData(void* InOutBuffer, int64 InNumBytes, int64 InFromOffset, FCancellationCheckDelegate InCheckCancellationDelegate) = 0;

		/**
		 * This method shall return the total size of the file.
		 * If the size is only known after performing the first read, -1 may be returned until the size is known.
		 * If the file is unbounded -1 may be returned at all times.
		 */
		virtual int64 GetTotalFileSize() = 0;

		/**
		 * This method shall return the current file offset, which is initially zero unless the file
		 * has been opened such that the initial position for this reader is not zero.
		 * The return value is the absolute file position the next read would occur at.
		 */
		virtual int64 GetCurrentFileOffset() = 0;

		/**
		 * A convenience method to return `true` when all data has been read.
		 * There is no requirement that `GetTotalFileSize()` returns a positive value.
		 * This method must return `true` when `ReadData()` would return `EResult::ReachedEOF`.
		 */
		virtual bool HasReachedEOF() = 0;

		/**
		 * If a read error occurred and `ReadData()` returns `EResult::ReadError` a human-readable
		 * message of what caused the error should be returned.
		 */
		virtual FString GetLastError() = 0;
	};

} // namespace Electra
