// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"

namespace Electra
{

	namespace ISO639
	{
		ELECTRABASE_API const TCHAR* Get639_1(const FString& InFrom639_1_2_3);
		ELECTRABASE_API FString MapTo639_1(const FString& InFrom639_1_2_3);

		ELECTRABASE_API FString RFC5646To639_1(const FString& InFromRFC5646);
	}

}
