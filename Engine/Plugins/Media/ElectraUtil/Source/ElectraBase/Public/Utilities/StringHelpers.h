// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>


namespace Electra
{

	inline void LexFromStringHex(int32& OutValue, const TCHAR* Buffer) { OutValue = FCString::Strtoi(Buffer, nullptr, 16); }
	inline void LexFromStringHex64(int64& OutValue, const TCHAR* Buffer) { OutValue = FCString::Strtoi64(Buffer, nullptr, 16); }
	inline void LexFromStringHexU64(uint64& OutValue, const TCHAR* Buffer) { OutValue = FCString::Strtoui64(Buffer, nullptr, 16); }

	namespace StringHelpers
	{
		FString ELECTRABASE_API ISO_8859_1_ToFString(const uint8* InStringToConvert, int32 InNumCharsToConvert=-1);

		int32 ELECTRABASE_API FindFirstOf(const FString& InString, const FString& SplitAt, int32 FirstPos = 0);

		int32 ELECTRABASE_API FindFirstNotOf(const FString& InString, const FString& InNotOfChars, int32 FirstPos = 0);

		int32 ELECTRABASE_API FindLastNotOf(const FString& InString, const FString& InNotOfChars, int32 StartPos = MAX_int32);

		void ELECTRABASE_API SplitByDelimiter(TArray<FString>& OutSplits, const FString& InString, const FString& SplitAt);

		bool ELECTRABASE_API StringEquals(const TCHAR * const s1, const TCHAR * const s2);

		bool ELECTRABASE_API StringStartsWith(const TCHAR * const s1, const TCHAR * const s2, SIZE_T n);

		void ELECTRABASE_API StringToArray(TArray<uint8>& OutArray, const FString& InString);

		FString ELECTRABASE_API ArrayToString(const TArray<uint8>& InArray);

		bool ELECTRABASE_API ArrayToString(FString& OutString, const TConstArrayView<const uint8>& InArray);

		FString ELECTRABASE_API GetLongestCommonPrefix(TArray<FString>& InOutTempArrayOfInputs);

		/**
		 * There is a known anomaly in the FString::TConstIterator. It iterates all TCHARs in the string *including* the terminating zero character.
		 * This is not the behaviour we want and setup some helper iterator here which is not including the terminating zero.
		 */
		class FStringIterator
		{
		public:
			FStringIterator(const FString& InString, int32 StartIndex = 0)
				: StringToIterate(InString)
				, Index(StartIndex)
			{
			}

			/** Advances iterator to the next element in the container. */
			FStringIterator& operator++()
			{
				++Index;
				return *this;
			}
			FStringIterator operator++(int)
			{
				FStringIterator Tmp(*this);
				++Index;
				return Tmp;
			}

			/** Moves iterator to the previous element in the container. */
			FStringIterator& operator--()
			{
				--Index;
				return *this;
			}
			FStringIterator operator--(int)
			{
				FStringIterator Tmp(*this);
				--Index;
				return Tmp;
			}

			/** iterator arithmetic support */
			FStringIterator& operator+=(int32 Offset)
			{
				Index += Offset;
				return *this;
			}

			FStringIterator operator+(int32 Offset) const
			{
				FStringIterator Tmp(*this);
				return Tmp += Offset;
			}

			FStringIterator& operator-=(int32 Offset)
			{
				return *this += -Offset;
			}

			FStringIterator operator-(int32 Offset) const
			{
				FStringIterator Tmp(*this);
				return Tmp -= Offset;
			}

			const TCHAR& operator* () const
			{
				return StringToIterate[Index];
			}

			const TCHAR* operator->() const
			{
				return &StringToIterate[Index];
			}

			/** conversion to "bool" returning true if the iterator has not reached the last element. */
			FORCEINLINE explicit operator bool() const
			{
				return Index >= 0 && Index < StringToIterate.Len();
			}

			const TCHAR* GetRemainder() const
			{
				return &StringToIterate[Index];
			}

			int32 GetRemainingLength() const
			{
				return StringToIterate.Len() - Index;
			}

			/** Returns an index to the current element. */
			int32 GetIndex() const
			{
				return Index;
			}

			/** Resets the iterator to the first element. */
			void Reset()
			{
				Index = 0;
			}

			/** Sets iterator to the last element. */
			void SetToEnd()
			{
				Index = StringToIterate.Len();
			}

			FORCEINLINE friend bool operator==(const FStringIterator& Lhs, const FStringIterator& Rhs) { return &Lhs.StringToIterate == &Rhs.StringToIterate && Lhs.Index == Rhs.Index; }
			FORCEINLINE friend bool operator!=(const FStringIterator& Lhs, const FStringIterator& Rhs) { return &Lhs.StringToIterate != &Rhs.StringToIterate || Lhs.Index != Rhs.Index; }

		private:
			const FString&	StringToIterate;
			int32			Index;
		};

	} // namespace StringHelpers

} // namespace Electra
