// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>
#include "PlayerTime.h"


namespace Electra
{

	namespace ISO8601
	{
		bool ELECTRABASE_API ParseDateTime(FTimeValue& OutTimeValue, const FString& DateTime);
		bool ELECTRABASE_API ParseDuration(FTimeValue& OutTimeValue, const TCHAR* InDuration);
	}

	namespace RFC7231
	{
		bool ELECTRABASE_API ParseDateTime(FTimeValue& OutTimeValue, const FString& DateTime);
	}

	namespace RFC2326
	{
		bool ELECTRABASE_API ParseNPTTime(FTimeValue& OutTimeValue, const FString& NPTtime);
	}

	namespace UnixEpoch
	{
		bool ELECTRABASE_API ParseFloatString(FTimeValue& OutTimeValue, const FString& Seconds);
	}

	namespace RFC5905
	{
		bool ELECTRABASE_API ParseNTPTime(FTimeValue& OutTimeValue, uint64 NtpTimestampFormat);
	}

} // namespace Electra

