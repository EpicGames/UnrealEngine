// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Utilities/StringHelpers.h"

#define UE_API ELECTRABASE_API


namespace Electra
{

	class FURL_RFC3986
	{
	public:
		// Parses the given URL.
		UE_API bool Parse(const FString& InURL);
		// Returns the scheme, if present. Does not include the :// sequence.
		UE_API FString GetScheme() const;
		// Returns the host, if present.
		UE_API FString GetHost() const;
		// Returns the port, if present.
		UE_API FString GetPort() const;
		// Returns the path. Escape sequences will already be decoded.
		UE_API FString GetPath() const;
		// Returns the entire query string, if any, without the leading '?'. Escape sequences will still be present!
		UE_API FString GetQuery() const;
		// Returns the fragment, if any, without the leading '#'. Escape sequences will already be decoded.
		UE_API FString GetFragment() const;
		// Returns the full URL with or without query and fragment parts. Characters will be escaped if necessary.
		UE_API FString Get(bool bIncludeQuery=true, bool bIncludeFragment=true);
		// Returns the path (no scheme or host) with or without query and fragment parts. Characters will be escaped if necessary.
		UE_API FString GetPath(bool bIncludeQuery, bool bIncludeFragment);
		// Returns whether or not this URL is absolute (the scheme not being empty).
		UE_API bool IsAbsolute() const;
		// Returns the path as individual components. Like GetPath() the components will have escape sequences already decoded.
		UE_API bool GetPathComponents(TArray<FString>& OutPathComponents) const;
		// Returns the last path component (the "filename").
		UE_API FString GetLastPathComponent() const;
		// Resolves the given (relative) URL against this one, which will be modified.
		UE_API void ResolveWith(const FURL_RFC3986& Other);
		// Resolves a relative URL against this one.
		UE_API FURL_RFC3986& ResolveWith(const FString& InChildURL);
		// Resolves this URL (which should be relative) against the specified URL.
		UE_API FURL_RFC3986& ResolveAgainst(const FString& InParentURL);

		// Attempts to make an absolute URL relative to this one.
		UE_API bool MakeRelativePath(FString& OutRelativePath, const FString& InAbsoluteURLToMakeRelativeToThis) const;

		UE_API void SetScheme(const FString& InNewScheme);
		UE_API void SetHost(const FString& InNewHost);
		UE_API void SetPort(const FString& InNewPort);
		UE_API void SetPath(const FString& InNewPath);
		UE_API void SetQuery(const FString& InNewQuery);
		UE_API void SetFragment(const FString& InNewFragment);

		// Returns if this URL has the same origin as another one as per RFC 6454.
		UE_API bool HasSameOriginAs(const FURL_RFC3986& Other);

		struct FQueryParam
		{
			FString Name;
			FString Value;
		};
		// Returns the query parameters as a list of name/value pairs.
		UE_API void GetQueryParams(TArray<FQueryParam>& OutQueryParams, bool bPerformUrlDecoding, bool bSameNameReplacesValue=true);
		// Returns the given query parameter string as a list of name/value pairs.
		// The parameter string MUST NOT start with a '?'.
		static UE_API void GetQueryParams(TArray<FQueryParam>& OutQueryParams, const FString& InQueryParameters, bool bPerformUrlDecoding, bool bSameNameReplacesValue=true);

		// Replaces query parameters in the URL.
        UE_API void SetQueryParams(const TArray<FQueryParam>& InQueryParams);

		// Updates existing parameters with new value or adds new parameters.
        UE_API void AddOrUpdateQueryParams(const TArray<FQueryParam>& InQueryParams);
		UE_API void AddOrUpdateQueryParams(const FString& InQueryParameters);

		// Decodes %XX escaped sequences into their original characters. Appends to the output. Hence in and out must not be the same.
		static UE_API bool UrlDecode(FString& OutResult, const FString& InUrlToDecode);
		// Encodes characters not permitted in a URL into %XX escaped sequences. Appends to the output. Hence in and out must not be the same.
		// A list of characters that should NOT be escaped, even if they normally would, can be passed. These characters must be strictly ASCII
		// characters only. This is useful to prevent '/' path delimiters to be escaped if the string to be encoded represents a path.
		static UE_API bool UrlEncode(FString& OutResult, const FString& InUrlToEncode, const FString& InCharsToKeep);

		// Returns the standard port for the given scheme. An empty string is returned if none is known.
		static UE_API FString GetStandardPortForScheme(const FString& InScheme, bool bIgnoreCase=true);

		// Returns a string containing the "sub-delims" chars that are permitted in the query string that do not
		// need to be escaped.
		static UE_API FString GetUrlEncodeSubDelimsChars();

	private:
		FString Scheme;
		FString UserInfo;
		FString Host;
		FString Port;
		FString Path;
		FString Query;
		FString Fragment;
		bool bIsFile = false;
		bool bIsData = false;

		static void GetPathComponents(TArray<FString>& OutPathComponents, const FString& InPath);

		void Empty();
		void Swap(FURL_RFC3986& Other);
		inline bool IsColonSeparator(TCHAR c)
		{ return c == TCHAR(':'); }
		inline bool IsPathSeparator(TCHAR c)
		{ return c == TCHAR('/'); }
		inline bool IsQuerySeparator(TCHAR c)
		{ return c == TCHAR('?'); }
		inline bool IsFragmentSeparator(TCHAR c)
		{ return c == TCHAR('#'); }
		inline bool IsQueryOrFragmentSeparator(TCHAR c)
		{ return IsQuerySeparator(c) || IsFragmentSeparator(c); }

		bool ParseAuthority(StringHelpers::FStringIterator& it);
		bool ParseHostAndPort(StringHelpers::FStringIterator& it);
		bool ParsePathAndQueryFragment(StringHelpers::FStringIterator& it);
		bool ParsePath(StringHelpers::FStringIterator& it);
		bool ParseQuery(StringHelpers::FStringIterator& it);
		bool ParseFragment(StringHelpers::FStringIterator& it);

		FString GetAuthority() const;
		void BuildPathFromSegments(const TArray<FString>& Components, bool bAddLeadingSlash, bool bAddTrailingSlash);
		void MergePath(const FString& InPathToMerge);
		void RemoveDotSegments();
	};



} // namespace Electra


#undef UE_API
