// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>

namespace Electra
{
	// Advance a pointer by a number of bytes.
	template <typename T, typename C>
	T AdvancePointer(T pPointer, C numBytes)
	{
		return(T(UPTRINT(pPointer) + UPTRINT(numBytes)));
	}

	namespace Utils
	{
		template <typename T>
		inline T AbsoluteValue(T Value)
		{
			return Value >= T(0) ? Value : -Value;
		}

		template <typename T>
		inline T Min(T a, T b)
		{
			return a < b ? a : b;
		}

		template <typename T>
		inline T Max(T a, T b)
		{
			return a > b ? a : b;
		}

		inline uint32 BitReverse32(uint32 InValue)
		{
			uint32 rev = 0;
			for(int32 i=0; i<32; ++i)
			{
				rev = (rev << 1) | (InValue & 1);
				InValue >>= 1;
			}
			return rev;
		}

		inline uint16 EndianSwap(uint16 value)
		{ 
			return (value >> 8) | (value << 8);
		}
		inline int16 EndianSwap(int16 value)
		{
			return int16(EndianSwap(uint16(value)));
		}
		inline uint32 EndianSwap(uint32 value)
		{ 
			return (value << 24) | ((value & 0xff00) << 8) | ((value >> 8) & 0xff00) | (value >> 24);
		}
		inline int32 EndianSwap(int32 value)
		{ 
			return int32(EndianSwap(uint32(value)));
		}
		inline uint64 EndianSwap(uint64 value)
		{ 
			return (uint64(EndianSwap(uint32(value & 0xffffffffU))) << 32) | uint64(EndianSwap(uint32(value >> 32)));
		}
		inline int64 EndianSwap(int64 value)
		{
			return int64(EndianSwap(uint64(value)));
		}

		inline constexpr uint32 Make4CC(const uint8 A, const uint8 B, const uint8 C, const uint8 D)
		{
			return (static_cast<uint32>(A) << 24) | (static_cast<uint32>(B) << 16) | (static_cast<uint32>(C) << 8) | static_cast<uint32>(D);
		}

	} // namespace Utils
} // namespace Electra
