// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>
#include "Misc/Variant.h"

#define UE_API ELECTRABASE_API

namespace Electra
{
	namespace MPEG
	{
		class FESDescriptor
		{
		public:
			UE_API FESDescriptor();
			UE_API void SetRawData(const void* Data, int64 Size);
			UE_API const TArray<uint8>& GetRawData() const;

			UE_API bool Parse();

			UE_API const TArray<uint8>& GetCodecSpecificData() const;

			uint32 GetBufferSize() const
			{
				return BufferSize;
			}

			uint32 GetMaxBitrate() const
			{
				return MaxBitrate;
			}

			uint32 GetAvgBitrate() const
			{
				return AvgBitrate;
			}

			// See http://mp4ra.org/#/object_types
			enum class FObjectTypeID
			{
				Unknown = 0,
				Text_Stream = 8,
				MPEG4_Video = 0x20,
				H264 = 0x21,
				H264_ParameterSets = 0x22,
				H265 = 0x23,
				MPEG4_Audio = 0x40,
				MPEG1_Audio = 0x6b
			};
			enum class FStreamType
			{
				Unknown = 0,
				VisualStream = 4,
				AudioStream = 5,
			};

			FObjectTypeID GetObjectTypeID() const
			{
				return ObjectTypeID;
			}

			FStreamType GetStreamType() const
			{
				return StreamTypeID;
			}

		private:
			TArray<uint8>						RawData;

			TArray<uint8>						CodecSpecificData;
			FObjectTypeID						ObjectTypeID;
			FStreamType							StreamTypeID;
			uint32								BufferSize;
			uint32								MaxBitrate;
			uint32								AvgBitrate;
			uint16								ESID;
			uint16								DependsOnStreamESID;
			uint8								StreamPriority;
			bool								bDependsOnStream;
		};



		class FID3V2Metadata
		{
		public:
			struct FItem
			{
				FString Language;				// ISO 639-2; if not set (all zero) the default entry for all languages
				FString MimeType;				// Mime type or Owner ID for private item.
				FVariant Value;
				int32 ItemType = -1;
			};

			UE_API bool Parse(const uint8* InData, int64 InDataSize);
			UE_API bool HaveTag(uint32 InTag);
			UE_API bool GetTag(FItem& OutValue, uint32 InTag);
			UE_API const TMap<uint32, FItem>& GetTags() const;
			UE_API TMap<uint32, FItem>& GetTags();
			UE_API const TArray<FItem>& GetPrivateItems() const;
		private:
			TMap<uint32, FItem> Tags;
			TArray<FItem> PrivateItems;
		};

	} // namespace MPEG
} // namespace Electra

#undef UE_API
