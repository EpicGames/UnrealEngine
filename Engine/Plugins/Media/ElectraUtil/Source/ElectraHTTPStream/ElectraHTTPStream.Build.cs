// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ElectraHTTPStream : ModuleRules
{
	public ElectraHTTPStream(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core"
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ElectraBase",
				"HTTP"
			});
	}
}
