// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraHTTPStream.h"
#include "Generic/GenericElectraHTTPStream.h"

TSharedPtr<IElectraHTTPStream, ESPMode::ThreadSafe> IElectraHTTPStream::Create(const Electra::FParamDict& InOptions)
{
	return FPlatformElectraHTTPStream::Create(InOptions);
}

