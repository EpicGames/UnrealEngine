// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraHTTPStreamModule.h"

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "IElectraHTTPStreamModule.h"

#include "Generic/GenericElectraHTTPStream.h"

#define LOCTEXT_NAMESPACE "ElectraHTTPStreamModule"

DEFINE_LOG_CATEGORY(LogElectraHTTPStream);

// -----------------------------------------------------------------------------------------------------------------------------------

class FElectraHTTPStreamModule : public IElectraHTTPStreamModule
{
public:
	// IModuleInterface interface

	void StartupModule() override
	{
		FPlatformElectraHTTPStream::Startup();
	}

	void ShutdownModule() override
	{
		FPlatformElectraHTTPStream::Shutdown();
	}

private:
};

IMPLEMENT_MODULE(FElectraHTTPStreamModule, ElectraHTTPStream);

#undef LOCTEXT_NAMESPACE
