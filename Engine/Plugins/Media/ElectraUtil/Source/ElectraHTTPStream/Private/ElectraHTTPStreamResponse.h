// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Templates/SharedPointer.h"

#include "ElectraHTTPStream.h"
#include "ElectraHTTPStreamBuffer.h"

class FElectraHTTPStreamResponse : public IElectraHTTPStreamResponse
{
public:
	FElectraHTTPStreamResponse() = default;
	virtual ~FElectraHTTPStreamResponse() = default;

	EStatus GetStatus() override
	{ return CurrentStatus; }

	EState GetState() override
	{ return CurrentState; }

	FString GetErrorMessage() override
	{ 
		FScopeLock lock(&Lock);
		return ErrorMessage; 
	}

	int32 GetHTTPResponseCode() override
	{ return HTTPResponseCode; }

	int64 GetNumResponseBytesReceived() override
	{ return ResponseData.GetNumTotalBytesAdded(); }

	int64 GetNumRequestBytesSent() override
	{ return 0;	}

	FString GetEffectiveURL() override
	{ return EffectiveURL; }

	void GetAllHeaders(TArray<FElectraHTTPStreamHeader>& OutHeaders) override
	{ OutHeaders = ResponseHeaders;	}

	FString GetHTTPStatusLine() override
	{ return HTTPStatusLine; }
	FString GetContentLengthHeader() override
	{ return ContentLength; }
	FString GetContentRangeHeader() override
	{ return ContentRange; }
	FString GetAcceptRangesHeader() override
	{ return AcceptRanges; }
	FString GetTransferEncodingHeader() override
	{ return TransferEncoding; }
	FString GetContentTypeHeader() override
	{ return ContentType; }

	IElectraHTTPStreamBuffer& GetResponseData() override
	{ return ResponseData; }

	double GetTimeElapsed() override
	{ return CurrentStatus == EStatus::NotRunning ? 0.0 : TimeUntilFinished > 0.0 ? TimeUntilFinished : FPlatformTime::Seconds() - StartTime; }

	double GetTimeSinceLastDataArrived() override
	{ return CurrentStatus == EStatus::NotRunning || TimeOfMostRecentReceive == 0.0 || TimeUntilFinished > 0.0 ? 0.0 : FPlatformTime::Seconds() - TimeOfMostRecentReceive; }

	double GetTimeUntilNameResolved() override
	{ return TimeUntilNameResolved; }
	double GetTimeUntilConnected() override
	{ return TimeUntilConnected; }
	double GetTimeUntilRequestSent() override
	{ return TimeUntilRequestSent; }
	double GetTimeUntilHeadersAvailable() override
	{ return TimeUntilHeadersAvailable; }
	double GetTimeUntilFirstByte() override
	{ return TimeUntilFirstByte; }
	double GetTimeUntilFinished() override
	{ return TimeUntilFinished; }

	int32 GetTimingTraces(TArray<FTimingTrace>* OutTraces, int32 InNumToRemove) override
	{
		FScopeLock lock(&Lock);
		int32 Num = TimingTraces.Num();
		if (OutTraces)
		{
			OutTraces->Append(TimingTraces);
		}
		if (InNumToRemove > 0)
		{
			int32 nr = InNumToRemove > Num ? Num : InNumToRemove;
			TimingTraces.RemoveAt(0, nr);
			Num = OutTraces ? Num : nr;	// If only removing the return value is to indicate how many got removed.
		}
		return Num;
	}

	void SetEnableTimingTraces() override
	{
		bCollectTimingTraces = true;
	}


	template<typename T>
	void AddTrace(T InNewData, bool bAddTrace)
	{
		if (bAddTrace && bCollectTimingTraces)
		{
			FTimingTrace Trace;
			Trace.TimeSinceStart = FPlatformTime::Seconds() - StartTime;
			Trace.NumBytesAdded = InNewData.Num();
			Trace.TotalBytesAdded = ResponseData.GetNumTotalBytesAdded() + InNewData.Num();
			FScopeLock lock(&Lock);
			TimingTraces.Emplace(MoveTemp(Trace));
		}
	}


	void AddResponseData(const TArray<uint8>& InNewData, bool bAddTrace=true)
	{ 
		AddTrace(InNewData, bAddTrace);
		ResponseData.AddData(InNewData); 
	}
	void AddResponseData(TArray<uint8>&& InNewData, bool bAddTrace=true)
	{ 
		AddTrace(InNewData, bAddTrace);
		ResponseData.AddData(MoveTemp(InNewData)); 
	}
	void AddResponseData(const TConstArrayView<const uint8>& InNewData, bool bAddTrace=true)
	{ 
		AddTrace(InNewData, bAddTrace);
		ResponseData.AddData(InNewData); 
	}


	void SetEOS()
	{ ResponseData.SetEOS(); }

	void SetErrorMessage(const FString& InErrorMessage)
	{
		FScopeLock lock(&Lock);
		ErrorMessage = InErrorMessage;
	}

	EStatus CurrentStatus = EStatus::NotRunning;
	EState CurrentState = EState::Connecting;

	TArray<FElectraHTTPStreamHeader> ResponseHeaders;
	FString ContentLength;
	FString ContentRange;
	FString AcceptRanges;
	FString TransferEncoding;
	FString ContentType;

	FString HTTPStatusLine;
	FString EffectiveURL;
	int32 HTTPResponseCode = 0;

	FElectraHTTPStreamBuffer ResponseData;

	// Timing
	double StartTime = 0.0;
	double TimeUntilNameResolved = 0.0;
	double TimeUntilConnected = 0.0;
	double TimeUntilRequestSent = 0.0;
	double TimeUntilHeadersAvailable = 0.0;
	double TimeUntilFirstByte = 0.0;
	double TimeUntilFinished = 0.0;
	double TimeOfMostRecentReceive = 0.0;

	FCriticalSection Lock;

	// Error message.
	FString ErrorMessage;

	// Timing traces.
	TArray<FTimingTrace> TimingTraces;
	bool bCollectTimingTraces = false;
};
