// Copyright Epic Games, Inc. All Rights Reserved.

#include "Generic/GenericElectraHTTPStream.h"

#include "Http.h"

#include "HAL/IConsoleManager.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/LowLevelMemTracker.h"

#include "ElectraHTTPStreamModule.h"
#include "ElectraHTTPStream.h"
#include "ElectraHTTPStreamBuffer.h"
#include "ElectraHTTPStreamResponse.h"
#include "ElectraHTTPStreamPerf.h"

#include "Utilities/TimeWaitableSignal.h"
#include "Utilities/DefaultHttpUserAgent.h"
#include "Utilities/HttpRangeHeader.h"
#include "Utilities/URLParser.h"

#include <atomic>

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

DECLARE_STATS_GROUP(TEXT("Electra HTTP Stream"), STATGROUP_ElectraHTTPStream, STATCAT_Advanced);
DECLARE_CYCLE_STAT(TEXT("Process"), STAT_ElectraHTTPThread_Process, STATGROUP_ElectraHTTPStream);
DECLARE_CYCLE_STAT(TEXT("Custom handler"), STAT_ElectraHTTPThread_CustomHandler, STATGROUP_ElectraHTTPStream);

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

class FElectraHTTPStreamRequestGeneric;

namespace ElectraHTTPStreamGeneric
{
	void LogError(const FString& Message)
	{
		UE_LOGF(LogElectraHTTPStream, Error, "%ls", *Message);
	}
}

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

/**
 * Generic Unreal Engine HTTP version of the ElectraHTTPStream implementation.
 */
class FElectraHTTPStreamGeneric : public TSharedFromThis<FElectraHTTPStreamGeneric, ESPMode::ThreadSafe>, public IElectraHTTPStream, private FRunnable
{
public:
	virtual ~FElectraHTTPStreamGeneric();

	FElectraHTTPStreamGeneric();
	bool Initialize(const Electra::FParamDict& InOptions);

	void AddThreadHandlerDelegate(FElectraHTTPStreamThreadHandlerDelegate InDelegate) override
	{
		FScopeLock lock(&CallbackLock);
		ThreadHandlerCallback = MoveTemp(InDelegate);
	}
	void RemoveThreadHandlerDelegate() override
	{
		FScopeLock lock(&CallbackLock);
		ThreadHandlerCallback.Unbind();
	}

	void Close() override;

	IElectraHTTPStreamRequestPtr CreateRequest() override;

	void AddRequest(IElectraHTTPStreamRequestPtr Request) override;

	void TriggerWorkSignal() override
	{ HaveWorkSignal.Signal(); }

private:
	// Methods from FRunnable
	uint32 Run() override final;
	void Stop() override final;

	void SetupNewRequests();
	void UpdateActiveRequests();
	void HandleCompletedRequests();

	FThreadSafeCounter ExitRequest;
	FRunnableThread* Thread = nullptr;
	FTimeWaitableSignal HaveWorkSignal;

	FCriticalSection CallbackLock;
	FElectraHTTPStreamThreadHandlerDelegate ThreadHandlerCallback;

	FCriticalSection RequestLock;
	TArray<TSharedPtr<FElectraHTTPStreamRequestGeneric, ESPMode::ThreadSafe>> NewRequests;
	TArray<TSharedPtr<FElectraHTTPStreamRequestGeneric, ESPMode::ThreadSafe>> ActiveRequests;
	TArray<TSharedPtr<FElectraHTTPStreamRequestGeneric, ESPMode::ThreadSafe>> CompletedRequests;
	TArray<TSharedPtr<FElectraHTTPStreamRequestGeneric, ESPMode::ThreadSafe>> CanceledRequests;
};

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

/**
 * Generic version of a HTTP stream request.
 */
class FElectraHTTPStreamRequestGeneric : public TSharedFromThis<FElectraHTTPStreamRequestGeneric, ESPMode::ThreadSafe>, public IElectraHTTPStreamRequest
{
public:
	enum EState
	{
		Inactive,
		Started,
		ReceivingHeaders,
		ReadingResponseData,
		Finished,
		Error,
	};

	FElectraHTTPStreamRequestGeneric(TSharedPtr<FElectraHTTPStreamGeneric, ESPMode::ThreadSafe> InOwner);
	virtual ~FElectraHTTPStreamRequestGeneric();

	void SetVerb(const FString& InVerb) override
	{ Verb = InVerb; }

	void EnableTimingTraces() override
	{ Response->SetEnableTimingTraces(); }

	IElectraHTTPStreamBuffer& POSTDataBuffer() override
	{ return PostData; }

	void SetUserAgent(const FString& InUserAgent) override
	{ UserAgent = InUserAgent; }

	void SetURL(const FString& InURL) override
	{ URL = InURL; }
	void SetRange(const FString& InRange) override
	{ Range = InRange; }
	void AllowCompression(bool bInAllowCompression) override
	{ bAllowCompression = bInAllowCompression; }

	void AllowUnsafeRequestsForDebugging() override
	{ }

	void AddHeader(const FString& Header, const FString& Value, bool bAppendIfExists) override
	{
		// Ignore a few headers that will be set automatically.
		if (Header.Equals(TEXT("User-Agent"), ESearchCase::IgnoreCase) ||
			Header.Equals(TEXT("Accept-Encoding"), ESearchCase::IgnoreCase) ||
			Header.Equals(TEXT("Transfer-Encoding"), ESearchCase::IgnoreCase) ||
			Header.Equals(TEXT("Range"), ESearchCase::IgnoreCase) ||
			Header.Equals(TEXT("Accept-Ranges"), ESearchCase::IgnoreCase))
		{
			return;
		}
		FString* ExistingHeader = AdditionalHeaders.Find(Header);
		if (ExistingHeader)
		{
			// If the value is empty and we shall replace the header we need to remove it since empty headers are prohibited.
			if (Value.IsEmpty() && !bAppendIfExists)
			{
				AdditionalHeaders.Remove(Header);
			}
			else if (Value.Len())
			{
				FString NewValue = *ExistingHeader;
				NewValue.Append(TEXT(", "));
				NewValue.Append(Value);
				*ExistingHeader = NewValue;
			}
		}
		// Header does not exist yet. If the value is empty we must not add the header since empty headers are prohibited.
		else if (Value.Len())
		{
			AdditionalHeaders.Add(Header, Value);
		}
	}

	FElectraHTTPStreamNotificationDelegate& NotificationDelegate() override
	{ return NotificationCallback; }

	void Cancel() override
	{
		bCancel = true;
		FScopeLock lock(&NotificationLock);
		NotificationCallback.Unbind();
	}

	IElectraHTTPStreamResponsePtr GetResponse() override
	{ return Response; }

	bool HasCompleted()
	{ return CurrentState == EState::Finished;	}

	bool HasFailed() override
	{ return Response->GetErrorMessage().Len() > 0; }

	FString GetErrorMessage() override
	{ return Response->GetErrorMessage(); }

	EState GetCurrentState()
	{ return CurrentState; }

	bool WasCanceled()
	{ return bCancel; }
	void CancelRunning()
	{
		if (!bWasCanceled && RequestHandle.IsValid())
		{
			RequestHandle->CancelRequest();
			bWasCanceled = true;
		}
	}

	bool Setup();
	bool Execute();
	bool ParseResponseHeaders();
	void SetFinished();

	void NotifyCallback(EElectraHTTPStreamNotificationReason InReason, int64 InParam)
	{
		FScopeLock lock(&NotificationLock);
		NotificationCallback.ExecuteIfBound(AsShared(), InReason, InParam);
	}

	void Close();
	void Terminate();

private:
	FElectraHTTPStreamRequestGeneric() = delete;
	FElectraHTTPStreamRequestGeneric(const FElectraHTTPStreamRequestGeneric&) = delete;
	FElectraHTTPStreamRequestGeneric& operator=(const FElectraHTTPStreamRequestGeneric&) = delete;

	void OnProcessRequestComplete(FHttpRequestPtr InSourceHttpRequest, FHttpResponsePtr InHttpResponse, bool bInSucceeded);
	void OnHeaderReceived(FHttpRequestPtr InSourceHttpRequest, const FString& InHeaderName, const FString& InHeaderValue);
	void OnStatusCodeReceived(FHttpRequestPtr InSourceHttpRequest, int32 InHttpStatusCode);
	void OnProcessRequestStream(void *InDataPtr, int64& InLength);

	// Owner to be notified on activity.
	TWeakPtr<FElectraHTTPStreamGeneric, ESPMode::ThreadSafe> Owner;

	// User agent. Defaults to a global one but can be set with each individual request.
	FString UserAgent;
	// GET or POST
	FString Verb;
	// URL to request
	FString URL;
	// Optional byte range. If set this must be a valid range string.
	FString Range;
	// Set to true to allow gzip/deflate for GET requests.
	bool bAllowCompression = false;
	// Additional headers to be sent with the request.
	TMap<FString, FString> AdditionalHeaders;

	FCriticalSection NotificationLock;
	FElectraHTTPStreamNotificationDelegate NotificationCallback;

	TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> RequestHandle;

	volatile EState CurrentState = EState::Inactive;
	std::atomic_bool bCancel { false };
	std::atomic_bool bWasCanceled { false };

	FElectraHTTPStreamBuffer PostData;
	TSharedPtr<FElectraHTTPStreamResponse, ESPMode::ThreadSafe> Response;

	FString EffectiveURL;
	int32 ReceivedHttpStatusCode = -1;
};

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

FElectraHTTPStreamRequestGeneric::FElectraHTTPStreamRequestGeneric(TSharedPtr<FElectraHTTPStreamGeneric, ESPMode::ThreadSafe> InOwner)
{
	Owner = InOwner;
	UserAgent = ElectraHTTPStream::GetDefaultUserAgent();
	Response = MakeShared<FElectraHTTPStreamResponse, ESPMode::ThreadSafe>();
}

FElectraHTTPStreamRequestGeneric::~FElectraHTTPStreamRequestGeneric()
{
	Close();
}

bool FElectraHTTPStreamRequestGeneric::Setup()
{
	// Check for a supported verb.
	if (Verb.IsEmpty())
	{
		Verb = TEXT("GET");
	}
	if (!(Verb.Equals(TEXT("GET")) || Verb.Equals(TEXT("POST")) || Verb.Equals(TEXT("HEAD"))))
	{
		Response->SetErrorMessage(FString::Printf(TEXT("Unsupported verb \"%s\""), *Verb));
		ElectraHTTPStreamGeneric::LogError(Response->GetErrorMessage());
		return false;
	}

	Electra::FURL_RFC3986 UrlParser;
	if (!UrlParser.Parse(URL))
	{
		Response->SetErrorMessage(FString::Printf(TEXT("Failed to parse URL \"%s\""), *URL));
		ElectraHTTPStreamGeneric::LogError(Response->GetErrorMessage());
		return false;
	}
	// Start out assuming the request URL will also be the effective URL.
	EffectiveURL = URL;

	RequestHandle = FHttpModule::Get().CreateRequest();
	RequestHandle->SetVerb(Verb);
	RequestHandle->SetURL(URL);
	RequestHandle->SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy::CompleteOnHttpThread);
	RequestHandle->SetPriority(EHttpRequestPriority::High);
	RequestHandle->OnProcessRequestComplete().BindThreadSafeSP(AsShared(), &FElectraHTTPStreamRequestGeneric::OnProcessRequestComplete);
	RequestHandle->OnHeaderReceived().BindThreadSafeSP(AsShared(), &FElectraHTTPStreamRequestGeneric::OnHeaderReceived);
	RequestHandle->OnStatusCodeReceived().BindThreadSafeSP(AsShared(), &FElectraHTTPStreamRequestGeneric::OnStatusCodeReceived);

	FHttpRequestStreamDelegateV2 StreamDelegate;
	StreamDelegate.BindThreadSafeSP(AsShared(), &FElectraHTTPStreamRequestGeneric::OnProcessRequestStream);
	bool bOk = RequestHandle->SetResponseBodyReceiveStreamDelegateV2(StreamDelegate);
	(void)bOk; check(bOk);

	// We set the user agent manually. For simplicities sake we add it to the list of additional headers.
	AdditionalHeaders.FindOrAdd(TEXT("User-Agent"), UserAgent);

	if (!bAllowCompression)
	{
		AdditionalHeaders.FindOrAdd(TEXT("Accept-Encoding"), TEXT("identity"));
	}

	// GET/HEAD / POST specific options
	if (Verb.Equals("GET") || Verb.Equals("HEAD"))
	{
		if (Range.Len())
		{
			FString RangeHdr = Range;
			if (!RangeHdr.StartsWith(TEXT("bytes=")))
			{
				RangeHdr.InsertAt(0, TEXT("bytes="));
			}
			AdditionalHeaders.FindOrAdd(TEXT("Range"), RangeHdr);
		}
	}

	// Add all additional headers.
	for(auto &Hdr : AdditionalHeaders)
	{
		RequestHandle->SetHeader(Hdr.Key, Hdr.Value);
	}

	return true;
}

bool FElectraHTTPStreamRequestGeneric::Execute()
{
	// Set the origin URL as effective URL first in case there are problems or no redirections.
	Response->EffectiveURL = EffectiveURL;
	Response->StartTime = FPlatformTime::Seconds();
	Response->CurrentStatus = IElectraHTTPStreamResponse::EStatus::Running;

	CurrentState = EState::Started;

	if (Verb.Equals("POST"))
	{
		// For now we need the EOS flag set as we send the data as a whole.
		check(PostData.GetEOS());
		const uint8* DataToSend;
		int64 NumDataToSend;
		PostData.LockBuffer(DataToSend, NumDataToSend);
		PostData.UnlockBuffer(0);

		TArray<uint8> UploadData(static_cast<const uint8*>(DataToSend), (int32)NumDataToSend);
		check(RequestHandle.IsValid());
		RequestHandle->SetContent(MoveTemp(UploadData));
	}

	check(RequestHandle.IsValid());
	if (RequestHandle.IsValid())
	{
		RequestHandle->ProcessRequest();
	}

	return true;
}

bool FElectraHTTPStreamRequestGeneric::ParseResponseHeaders()
{
	check(Response.IsValid());
	if (!Response.IsValid())
	{
		return false;
	}
	for(auto &Hdr : Response->ResponseHeaders)
	{
		// Check for the most commonly used headers and set them on the side for faster access.
		if (Hdr.Header.Equals(TEXT("Content-Type"), ESearchCase::IgnoreCase))
		{
			Response->ContentType = Hdr.Value;
		}
		else if (Hdr.Header.Equals(TEXT("Content-Length"), ESearchCase::IgnoreCase))
		{
			Response->ContentLength = Hdr.Value;
		}
		else if (Hdr.Header.Equals(TEXT("Content-Range"), ESearchCase::IgnoreCase))
		{
			Response->ContentRange = Hdr.Value;
		}
		else if (Hdr.Header.Equals(TEXT("Accept-Ranges"), ESearchCase::IgnoreCase))
		{
			Response->AcceptRanges = Hdr.Value;
		}
		else if (Hdr.Header.Equals(TEXT("Transfer-Encoding"), ESearchCase::IgnoreCase))
		{
			Response->TransferEncoding = Hdr.Value;
		}
	}
	// We do not have the full status line, so only use the response code.
	//Response->HTTPStatusLine = ;
	Response->HTTPResponseCode = ReceivedHttpStatusCode;

	// The effective URL is only known at the end of the transfer.
	//Response->EffectiveURL = ;

	// Pre-allocate the receive buffer unless this is a HEAD request for which we will not get any data.
	if (!Verb.Equals(TEXT("HEAD")))
	{
		// Check for Content-Length header
		if (Response->ContentLength.Len())
		{
			int64 cl = -1;
			LexFromString(cl, *Response->ContentLength);
			if (cl >= 0)
			{
				Response->ResponseData.SetLengthFromResponseHeader(cl);
			}
		}
		// Alternatively check Content-Range header
		else if (Response->ContentRange.Len())
		{
			ElectraHTTPStream::FHttpRange ContentRange;
			if (ContentRange.ParseFromContentRangeResponse(Response->ContentRange))
			{
				if (ContentRange.GetNumberOfBytes() >= 0)
				{
					Response->ResponseData.SetLengthFromResponseHeader(ContentRange.GetNumberOfBytes());
				}
			}
		}
	}

	// Notify availability of response headers.
	Response->CurrentState = IElectraHTTPStreamResponse::EState::ReceivedResponseHeaders;
	NotifyCallback(EElectraHTTPStreamNotificationReason::ReceivedHeaders, Response->ResponseHeaders.Num());

	return true;
}

void FElectraHTTPStreamRequestGeneric::Close()
{
	CancelRunning();
	RequestHandle.Reset();
}

void FElectraHTTPStreamRequestGeneric::Terminate()
{
	Close();
	Response->SetErrorMessage(TEXT("Terminated due to HTTP module shutdown"));
}

void FElectraHTTPStreamRequestGeneric::SetFinished()
{
	double Now = FPlatformTime::Seconds();
	Response->TimeUntilFinished = Now - Response->StartTime;
	Response->CurrentStatus = WasCanceled() ? IElectraHTTPStreamResponse::EStatus::Canceled : HasFailed() ?	IElectraHTTPStreamResponse::EStatus::Failed : IElectraHTTPStreamResponse::EStatus::Completed;
	Response->CurrentState = IElectraHTTPStreamResponse::EState::Finished;
	Response->SetEOS();
	CurrentState = HasFailed() ? EState::Error : EState::Finished;
	RequestHandle.Reset();
}


void FElectraHTTPStreamRequestGeneric::OnProcessRequestComplete(FHttpRequestPtr InSourceHttpRequest, FHttpResponsePtr InHttpResponse, bool bInSucceeded)
{
	// If we did not receive any response body data bytes yet (because this is a HEAD request or due to some problem), process the headers.
	if (CurrentState < EState::ReadingResponseData)
	{
		double Now = FPlatformTime::Seconds();
		CurrentState = EState::ReadingResponseData;
		Response->TimeUntilFirstByte = Now - Response->StartTime;
		// Parse the headers and report them.
		ParseResponseHeaders();
	}

	if (InHttpResponse.IsValid())
	{
		EffectiveURL = InHttpResponse->GetEffectiveURL();
		Response->HTTPResponseCode = InHttpResponse->GetResponseCode();
		Response->EffectiveURL = EffectiveURL;
	}
	if (!bInSucceeded && !WasCanceled())
	{
		if (Response->HTTPResponseCode)
		{
			if (Response->HTTPResponseCode >= 400)
			{
				Response->SetErrorMessage(FString::Printf(TEXT("Failed with HTTP status %d"), Response->HTTPResponseCode));
			}
			else
			{
				Response->SetErrorMessage(FString::Printf(TEXT("Failed due to connection error")));
			}
		}
		else
		{
			EHttpFailureReason fr = InHttpResponse.IsValid() ? InHttpResponse->GetFailureReason() : EHttpFailureReason::Other;
			switch(fr)
			{
				case EHttpFailureReason::ConnectionError:
				{
					Response->SetErrorMessage(FString::Printf(TEXT("Failed due to connection error")));
					break;
				}
				default:
				{
					Response->SetErrorMessage(FString::Printf(TEXT("Connection failed")));
					break;
				}
			}
		}
		ElectraHTTPStreamGeneric::LogError(Response->GetErrorMessage());
	}
	CurrentState = EState::Finished;

	TSharedPtr<FElectraHTTPStreamGeneric, ESPMode::ThreadSafe> PinnedOwner = Owner.Pin();
	if (PinnedOwner.IsValid())
	{
		PinnedOwner->TriggerWorkSignal();
	}
}

void FElectraHTTPStreamRequestGeneric::OnHeaderReceived(FHttpRequestPtr InSourceHttpRequest, const FString& InHeaderName, const FString& InHeaderValue)
{
	if (WasCanceled())
	{
		return;
	}
	double Now = FPlatformTime::Seconds();
	if (CurrentState < EState::ReceivingHeaders)
	{
		CurrentState = EState::ReceivingHeaders;
		Response->TimeUntilHeadersAvailable = Now - Response->StartTime;
		// Cannot get the timings of the individual steps preceeding getting the first header.
		// Calculate them as quarters of the time elapsed so far.
		Response->TimeUntilNameResolved = Response->TimeUntilHeadersAvailable * 0.25;
		Response->TimeUntilConnected = Response->TimeUntilHeadersAvailable * 0.5;
		Response->TimeUntilRequestSent = Response->TimeUntilHeadersAvailable * 0.75;
	}
	if (CurrentState == EState::ReceivingHeaders)
	{
		FElectraHTTPStreamHeader Hdr;
		Hdr.Header = InHeaderName;
		Hdr.Value = InHeaderValue;
		Response->ResponseHeaders.Emplace(MoveTemp(Hdr));
	}
	Response->TimeOfMostRecentReceive = Now;
}

void FElectraHTTPStreamRequestGeneric::OnStatusCodeReceived(FHttpRequestPtr InSourceHttpRequest, int32 InHttpStatusCode)
{
	if (InHttpStatusCode)
	{
		ReceivedHttpStatusCode = InHttpStatusCode;
	}
	if (WasCanceled())
	{
		return;
	}
	if (InHttpStatusCode)
	{
		// For the lack of better knowledge pretend this is a 1.1 transfer.
		FString HeaderValue = FString::Printf(TEXT("HTTP/1.1 %d"), InHttpStatusCode);
		OnHeaderReceived(InSourceHttpRequest, FString(), HeaderValue);
	}
}

void FElectraHTTPStreamRequestGeneric::OnProcessRequestStream(void *InDataPtr, int64& InOutLength)
{
	if (InDataPtr == nullptr || InOutLength < 0)
	{
		InOutLength = 0;
		return;
	}
	if (WasCanceled())
	{
		return;
	}
	double Now = FPlatformTime::Seconds();
	if (CurrentState < EState::ReadingResponseData)
	{
		CurrentState = EState::ReadingResponseData;
		Response->TimeUntilFirstByte = Now - Response->StartTime;
		// Parse the headers and report them.
		if (!ParseResponseHeaders())
		{
			InOutLength = 0;
			return;
		}
	}

	// Add the data to the response.
	TConstArrayView<const uint8> Data(static_cast<const uint8*>(InDataPtr), (int32)InOutLength);
	Response->AddResponseData(Data);
	// Notify amount of new data available.
	NotifyCallback(EElectraHTTPStreamNotificationReason::ReadData, InOutLength);

	Response->TimeOfMostRecentReceive = Now;

	TSharedPtr<FElectraHTTPStreamGeneric, ESPMode::ThreadSafe> PinnedOwner = Owner.Pin();
	if (PinnedOwner.IsValid())
	{
		PinnedOwner->TriggerWorkSignal();
	}
}


/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

FElectraHTTPStreamGeneric::FElectraHTTPStreamGeneric()
{
}

FElectraHTTPStreamGeneric::~FElectraHTTPStreamGeneric()
{
	Close();
}

bool FElectraHTTPStreamGeneric::Initialize(const Electra::FParamDict& InOptions)
{
	LLM_SCOPE(ELLMTag::MediaStreaming);
	// Create the worker thread.
	Thread = FRunnableThread::Create(this, TEXT("ElectraHTTPStream"), 128 * 1024, TPri_Normal);
	if (!Thread)
	{
		UE_LOGF(LogElectraHTTPStream, Error, "Failed to create the ElectraHTTPStream worker thread");
		return false;
	}
	return true;
}

void FElectraHTTPStreamGeneric::Close()
{
	LLM_SCOPE(ELLMTag::MediaStreaming);
	if (Thread)
	{
		Thread->Kill(true);
		delete Thread;
		Thread = nullptr;
	}
}


IElectraHTTPStreamRequestPtr FElectraHTTPStreamGeneric::CreateRequest()
{
	LLM_SCOPE(ELLMTag::MediaStreaming);
	return MakeShared<FElectraHTTPStreamRequestGeneric, ESPMode::ThreadSafe>(AsShared());
}

void FElectraHTTPStreamGeneric::AddRequest(IElectraHTTPStreamRequestPtr InRequest)
{
	if (InRequest.IsValid())
	{
		if (Thread)
		{
			FScopeLock lock(&RequestLock);
			NewRequests.Emplace(StaticCastSharedPtr<FElectraHTTPStreamRequestGeneric>(InRequest));
			TriggerWorkSignal();
		}
		else
		{
			TSharedPtr<FElectraHTTPStreamRequestGeneric, ESPMode::ThreadSafe> Req = StaticCastSharedPtr<FElectraHTTPStreamRequestGeneric>(InRequest);
			Req->Terminate();
			Req->NotifyCallback(EElectraHTTPStreamNotificationReason::Completed, 1);
		}
	}
}

void FElectraHTTPStreamGeneric::Stop()
{
	ExitRequest.Set(1);
}

uint32 FElectraHTTPStreamGeneric::Run()
{
	LLM_SCOPE(ELLMTag::MediaStreaming);

	while(!ExitRequest.GetValue())
	{
		HaveWorkSignal.WaitTimeoutAndReset(20);

		{
		SCOPE_CYCLE_COUNTER(STAT_ElectraHTTPThread_Process);
		SetupNewRequests();
		UpdateActiveRequests();
		HandleCompletedRequests();
		}

		// User callback
		{
		SCOPE_CYCLE_COUNTER(STAT_ElectraHTTPThread_CustomHandler);
		FScopeLock lock(&CallbackLock);
		ThreadHandlerCallback.ExecuteIfBound();
		}
	}

	// Remove requests.
	RequestLock.Lock();
	while(NewRequests.Num())
	{
		TSharedPtr<FElectraHTTPStreamRequestGeneric, ESPMode::ThreadSafe> Req = NewRequests.Pop();
		Req->Terminate();
		CompletedRequests.Emplace(MoveTemp(Req));
	}
	while(ActiveRequests.Num())
	{
		TSharedPtr<FElectraHTTPStreamRequestGeneric, ESPMode::ThreadSafe> Req = ActiveRequests.Pop();
		Req->Terminate();
		CanceledRequests.Emplace(MoveTemp(Req));
	}
	RequestLock.Unlock();
	while(CompletedRequests.Num() || CanceledRequests.Num())
	{
		HandleCompletedRequests();
		FPlatformProcess::Sleep(0.1f);
	}
	return 0;
}

void FElectraHTTPStreamGeneric::SetupNewRequests()
{
	RequestLock.Lock();
	TArray<TSharedPtr<FElectraHTTPStreamRequestGeneric, ESPMode::ThreadSafe>> NewReqs;
	Swap(NewReqs, NewRequests);
	RequestLock.Unlock();
	for(auto &Request : NewReqs)
	{
		if (Request->Setup())
		{
			ActiveRequests.Emplace(Request);
			if (Request->WasCanceled() || !Request->Execute())
			{
				ActiveRequests.Remove(Request);
				CompletedRequests.Emplace(Request);
			}
		}
		else
		{
			CompletedRequests.Emplace(Request);
		}
	}
}

void FElectraHTTPStreamGeneric::UpdateActiveRequests()
{
	for(int32 i=0; i<ActiveRequests.Num(); ++i)
	{
		TSharedPtr<FElectraHTTPStreamRequestGeneric, ESPMode::ThreadSafe> Request = ActiveRequests[i];
		bool bRemoveRequest = false;
		bool bErrored = false;

		// If the request got canceled do not handle it any further and move it to the completed stage.
		// When completed, move to finished.
		if (Request->HasCompleted())
		{
			bRemoveRequest = true;
		}
		// If the request has failed in any way, do not handle it any further and move it to the completed stage.
		else if (Request->HasFailed())
		{
			bErrored = true;
			bRemoveRequest = true;
		}
		else if (Request->WasCanceled())
		{
			Request->CancelRunning();
			ActiveRequests.RemoveAt(i);
			--i;
			CanceledRequests.Emplace(MoveTemp(Request));
		}
		if (bRemoveRequest)
		{
			ActiveRequests.RemoveAt(i);
			--i;
			CompletedRequests.Emplace(MoveTemp(Request));
		}
	}
}

void FElectraHTTPStreamGeneric::HandleCompletedRequests()
{
	for(int32 i=0; i<CanceledRequests.Num(); ++i)
	{
		TSharedPtr<FElectraHTTPStreamRequestGeneric, ESPMode::ThreadSafe> Request = CanceledRequests[i];
		if (Request->HasCompleted())
		{
			CanceledRequests.RemoveAt(i);
			--i;
			CompletedRequests.Emplace(MoveTemp(Request));
		}
	}

	if (CompletedRequests.Num())
	{
		TArray<TSharedPtr<FElectraHTTPStreamRequestGeneric, ESPMode::ThreadSafe>> TempRequests;
		Swap(CompletedRequests, TempRequests);
		for(auto &Finished : TempRequests)
		{
			Finished->SetFinished();
			Finished->Close();
			// Parameter is 0 when finished successfully or 1 otherwise
			int64 Param = Finished->HasFailed() ? 1 : 0;
			if (!Finished->WasCanceled())
			{
				Finished->NotifyCallback(EElectraHTTPStreamNotificationReason::Completed, Param);
			}
		}
	}
}


/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

void FPlatformElectraHTTPStreamGeneric::Startup()
{
}

void FPlatformElectraHTTPStreamGeneric::Shutdown()
{
}

TSharedPtr<IElectraHTTPStream, ESPMode::ThreadSafe> FPlatformElectraHTTPStreamGeneric::Create(const Electra::FParamDict& InOptions)
{
	TSharedPtr<FElectraHTTPStreamGeneric, ESPMode::ThreadSafe> New = MakeShareable(new FElectraHTTPStreamGeneric);
	if (New.IsValid())
	{
		if (!New->Initialize(InOptions))
		{
			New.Reset();
		}
	}
	return New;
}
