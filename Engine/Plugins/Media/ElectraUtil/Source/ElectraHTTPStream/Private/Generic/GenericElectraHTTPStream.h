// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "ParameterDictionary.h"

#define UE_API ELECTRAHTTPSTREAM_API

class IElectraHTTPStream;

class FPlatformElectraHTTPStreamGeneric
{
public:
	static UE_API void Startup();
	static UE_API void Shutdown();
	static UE_API TSharedPtr<IElectraHTTPStream, ESPMode::ThreadSafe> Create(const Electra::FParamDict& InOptions);
};

typedef FPlatformElectraHTTPStreamGeneric FPlatformElectraHTTPStream;

#undef UE_API
