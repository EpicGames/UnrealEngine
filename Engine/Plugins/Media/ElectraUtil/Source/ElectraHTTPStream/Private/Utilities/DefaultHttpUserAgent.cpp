// Copyright Epic Games, Inc. All Rights Reserved.


#include "Utilities/DefaultHttpUserAgent.h"

#include "Misc/EngineVersion.h"
#include "Misc/App.h"

namespace ElectraHTTPStream
{

static FString EscapeString(const FString& UnescapedString)
{
	if (UnescapedString.Contains(" ") || UnescapedString.Contains("/"))
	{
		FString EscapedString = UnescapedString;
		EscapedString.ReplaceInline(TEXT(" "), TEXT(""));
		EscapedString.ReplaceInline(TEXT("/"), TEXT("+"));
		return EscapedString;
	}
	else
	{
		return UnescapedString;
	}
}



FString GetDefaultUserAgent()
{
	//** strip/escape slashes and whitespace from components
	static FString CachedUserAgent = FString::Printf(TEXT("%s/%s %s/%s"),
		*EscapeString(FApp::GetProjectName()),
		*EscapeString(FApp::GetBuildVersion()),
		*EscapeString(FString(FPlatformProperties::IniPlatformName())),
		*EscapeString(FPlatformMisc::GetOSVersion()));
	return CachedUserAgent;
}

}
