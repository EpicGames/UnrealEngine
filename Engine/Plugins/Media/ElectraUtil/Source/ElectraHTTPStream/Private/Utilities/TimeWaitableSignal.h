// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"



class FTimeWaitableSignal
{
public:
	FTimeWaitableSignal() : Event(FPlatformProcess::GetSynchEventFromPool(true))
	{ }

	~FTimeWaitableSignal()
	{
		Event->Trigger();
		FPlatformProcess::ReturnSynchEventToPool(Event);
	}

	void Wait()
	{
		Event->Wait();
	}

	void WaitAndReset()
	{
		Wait();
		Reset();
	}

	void Signal()
	{
		Event->Trigger();
	}

	void Reset()
	{
		Event->Reset();
	}

	bool IsSignaled()
	{
		return Event->Wait(0);
	}

	bool WaitTimeout(int32 MilliSeconds)
	{
		return Event->Wait(FTimespan::FromMilliseconds(MilliSeconds));
	}

	bool WaitTimeoutAndReset(int32 MilliSeconds)
	{
		if (MilliSeconds <= 0)
		{
			WaitAndReset();
			return true;
		}
		else
		{
			bool bGot = Event->Wait(FTimespan::FromMilliseconds(MilliSeconds));
			if (bGot)
			{
				Event->Reset();
			}
			return bGot;
		}
	}

private:
	FEvent* Event;

};
