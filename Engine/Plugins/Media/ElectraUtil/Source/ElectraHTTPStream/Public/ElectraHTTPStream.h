// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "ParameterDictionary.h"

class IElectraHTTPStreamRequest;
using IElectraHTTPStreamRequestPtr = TSharedPtr<IElectraHTTPStreamRequest, ESPMode::ThreadSafe>;

class IElectraHTTPStreamResponse;
using IElectraHTTPStreamResponsePtr = TSharedPtr<IElectraHTTPStreamResponse, ESPMode::ThreadSafe>;



class IElectraHTTPStreamBuffer
{
public:
	IElectraHTTPStreamBuffer() = default;
	IElectraHTTPStreamBuffer(const IElectraHTTPStreamBuffer&) = delete;
	IElectraHTTPStreamBuffer& operator=(const IElectraHTTPStreamBuffer&) = delete;
	virtual ~IElectraHTTPStreamBuffer() = default;

	virtual void AddData(const TArray<uint8>& InNewData) = 0;
	virtual void AddData(TArray<uint8>&& InNewData) = 0;
	virtual void AddData(const TConstArrayView<const uint8>& InNewData) = 0;
	virtual void AddData(const IElectraHTTPStreamBuffer& Other, int64 Offset, int64 NumBytes) = 0;

	virtual int64 GetNumTotalBytesAdded() const = 0;
	virtual int64 GetNumTotalBytesHandedOut() const = 0;
	virtual int64 GetNumBytesAvailableForRead()	const = 0;

	virtual void LockBuffer(const uint8*& OutNextReadAddress, int64& OutNumBytesAvailable) = 0;
	virtual void UnlockBuffer(int64 NumBytesConsumed) = 0;
	virtual bool RewindToBeginning() = 0;

	virtual bool GetEOS() const = 0;
	virtual void SetEOS() = 0;
	virtual void ClearEOS() = 0;

	virtual bool HasAllDataBeenConsumed() const = 0;

	virtual bool IsCachable() const = 0;
	virtual void SetIsCachable(bool bInIsCachable) = 0;

	virtual void SetLengthFromResponseHeader(int64 InLengthFromResponseHeader) = 0;
	virtual int64 GetLengthFromResponseHeader() const = 0;

	virtual void GetBaseBuffer(const uint8*& OutBaseAddress, int64& OutBytesInBuffer) = 0;
};



enum class EElectraHTTPStreamNotificationReason
{
	ReceivedHeaders,
	ReadData,
	WriteData,
	Completed
};

DECLARE_DELEGATE_ThreeParams(FElectraHTTPStreamNotificationDelegate, IElectraHTTPStreamRequestPtr /*Request*/, EElectraHTTPStreamNotificationReason /*Reason*/, int64 /*Param*/);

struct FElectraHTTPStreamHeader
{
	FString Header;
	FString Value;
};


class IElectraHTTPStreamResponse
{
public:
	virtual ~IElectraHTTPStreamResponse() = default;

	enum class EStatus
	{
		NotRunning,
		Running,
		Completed,
		Canceled,
		Failed
	};

	// This enum needs to be kept in order of data flow so states can be compared with < and >
	enum class EState
	{
		Connecting,
		SendingRequestData,
		WaitingForResponseHeaders,
		ReceivedResponseHeaders,
		ReceivingResponseData,
		Finished
	};

	struct FTimingTrace
	{
		// Seconds since the request started.
		double TimeSinceStart;
		// Bytes added with the most recent chunk.
		int64 NumBytesAdded;
		// Total bytes added to the buffer including the most recent chunk.
		int64 TotalBytesAdded;
	};

	virtual EStatus GetStatus() = 0;
	virtual EState GetState() = 0;

	virtual FString GetErrorMessage() = 0;

	virtual int32 GetHTTPResponseCode() = 0;

	virtual int64 GetNumResponseBytesReceived() = 0;
	virtual int64 GetNumRequestBytesSent() = 0;

	virtual FString GetEffectiveURL() = 0;

	virtual void GetAllHeaders(TArray<FElectraHTTPStreamHeader>& OutHeaders) = 0;
	virtual FString GetHTTPStatusLine() = 0;
	virtual FString GetContentLengthHeader() = 0;
	virtual FString GetContentRangeHeader() = 0;
	virtual FString GetAcceptRangesHeader() = 0;
	virtual FString GetTransferEncodingHeader() = 0;
	virtual FString GetContentTypeHeader() = 0;

	virtual IElectraHTTPStreamBuffer& GetResponseData() = 0;

	virtual double GetTimeElapsed() = 0;
	virtual double GetTimeSinceLastDataArrived() = 0;
	
	// Static timings as progressing through states.
	virtual double GetTimeUntilNameResolved() = 0;
	virtual double GetTimeUntilConnected() = 0;
	virtual double GetTimeUntilRequestSent() = 0;
	virtual double GetTimeUntilHeadersAvailable() = 0;
	virtual double GetTimeUntilFirstByte() = 0;
	virtual double GetTimeUntilFinished() = 0;

	// Appends a copy of the timing traces to the provided array and optionally removes the first n elements.
	// Pass a nullptr to only remove elements without copying them. This is useful to first get the traces
	// and in a second call remove those that are no longer needed.
	// Returns number appended or removed.
	virtual int32 GetTimingTraces(TArray<FTimingTrace>* OutTraces, int32 InNumToRemove) = 0;
	virtual void SetEnableTimingTraces() = 0;
};



/**
 * HTTP request instance created by IElectraHTTPStream::CreateRequest()
 */
class IElectraHTTPStreamRequest
{
public:
	virtual ~IElectraHTTPStreamRequest() = default;

	// Set verb, GET, HEAD or POST. If not set GET is used by default.
	virtual void SetVerb(const FString& Verb) = 0;

	// Enables collection of timing traces. (disabled by default)
	virtual void EnableTimingTraces() = 0;

	/*
		Returns the buffer holding the data to be POSTed to the server.
		Make sure to include a suitable "Content-Type: xxx" header via AddHeader()
	*/
	virtual IElectraHTTPStreamBuffer& POSTDataBuffer() = 0;

	// Sets a user agent for this request. If not specified an application specific default is used.
	virtual void SetUserAgent(const FString& UserAgent) = 0;

	// Sets the URL to issue the request to. Must not contain an URL fragment and must be appropriately escaped.
	virtual void SetURL(const FString& URL) = 0;
	
	// Sets a range for a GET request. Must be a valid range string.
	virtual void SetRange(const FString& Range) = 0;
	
	// Allows for GET requests to return a compressed response using one of the platform supported algorithms.
	virtual void AllowCompression(bool bAllowCompression) = 0;
	
	// Call this to allow unsafe connections for debugging purposes. Not enabled in shipping builds.
	virtual void AllowUnsafeRequestsForDebugging() = 0;

	/*
		Adds a request header. Only one header of a type is allowed. If one already exists it will be replaced
		with the new value unless bAppendIfExists is true which will append the value to the existing value with a ", " sequence.
		An existing header can be removed by passing an empty value with bAppendIfExists set to false.
		This may not be useful since you can always just not add the header to begin with.
	*/
	virtual void AddHeader(const FString& Header, const FString& Value, bool bAppendIfExists) = 0;

	/*
		Returns the notification delegate to which you bind a progress callback.
		This must be used only to add a delegate before adding the request for execution.
		Accessing this delegate while the request is running is not considered thread safe.
		Do not unbind your callback! Use Cancel() instead.
	*/
	virtual FElectraHTTPStreamNotificationDelegate& NotificationDelegate() = 0;

	// Cancels the request and unbinds your notification delegate.
	virtual void Cancel() = 0;

	// Returns the response object which is created with this request immediately and always accessible.
	virtual IElectraHTTPStreamResponsePtr GetResponse() = 0;

	/*
		Returns whether the request has failed. Please note that HTTP responses in the 4xx and 5xx range
		are not failures since a communication was established and a response was received.
		Such errors must be detected and handled by your application examining the response status!
	*/
	virtual bool HasFailed() = 0;

	// Returns the reason the request has failed.
	virtual FString GetErrorMessage() = 0;
};



class IElectraHTTPStream
{
public:
	/**
	 * Creates an instance of the HTTP streamer class.
	 */
	ELECTRAHTTPSTREAM_API static TSharedPtr<IElectraHTTPStream, ESPMode::ThreadSafe> Create(const Electra::FParamDict& InOptions);

	virtual ~IElectraHTTPStream() = default;

	/**
	 * Adds a method of your application to be called periodically from the worker thread of this handler.
	 * You may use this to drive another wrapping handler of yours that controls requests.
	 * For direct use of requests generated by this handler it is not necessary to add such a method.
	 * The NotificationDelegate() of the IElectraHTTPStreamRequestPtr request object should be all you need.
	 */
	DECLARE_DELEGATE(FElectraHTTPStreamThreadHandlerDelegate);
	virtual void AddThreadHandlerDelegate(FElectraHTTPStreamThreadHandlerDelegate InDelegate) = 0;
	virtual void RemoveThreadHandlerDelegate() = 0;

	/**
	 * Triggers a call to the registered handler delegate without waiting for the worker thread's
	 * timeout period.
	 */
	virtual void TriggerWorkSignal() = 0;

	/**
	 * Closes this handler and all currently active or pending requests.
	 * This should be called before resetting the TSharedPtr<> returned by Create().
	 */
	virtual void Close() = 0;

	/**
	 * Creates a request. Use the request object's methods to configure the request with the necessary parameters
	 * and then pass the request for execution to AddRequest().
	 */
	virtual IElectraHTTPStreamRequestPtr CreateRequest() = 0;

	/**
	 * Adds the request for execution. A strong reference of the request is kept internally while the request is running.
	 * It is an error to drop the request object without calling Request->Cancel() if there is a notification delegate
	 * bound to the request. The delegate will be called from a worker thread even if you have released the request because
	 * of the strong reference being held internally. If your delegate callback object has been destroyed it may result in
	 * a crash. You must hold on to the request until the callback signals completion or you have to call Cancel().
	 * 
	 * If the request fails the registered notification delegate will be invoked with the cause set to 'Completed'
	 * and the parameter set to 1. The error cause can be retrieved from the request via GetErrorMessage().
	 * Please note that HTTP responses like 404, 503 etc. are not an error per se. Handling of such responses must
	 * be made by the caller.
	 */
	virtual void AddRequest(IElectraHTTPStreamRequestPtr Request) = 0;
};
