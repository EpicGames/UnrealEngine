// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ElectraHTTPStream.h"
#include <atomic>

class FElectraHTTPStreamBuffer : public IElectraHTTPStreamBuffer
{
public:
	FElectraHTTPStreamBuffer() = default;
	FElectraHTTPStreamBuffer(const FElectraHTTPStreamBuffer&) = delete;
	FElectraHTTPStreamBuffer& operator=(const FElectraHTTPStreamBuffer&) = delete;
	virtual ~FElectraHTTPStreamBuffer() = default;

	virtual void AddData(const TArray<uint8>& InNewData) override
	{
		FScopeLock lock(&Lock);
		NumBytesAdded += InNewData.Num();
		Buffer.Append(InNewData);
	}

	virtual void AddData(TArray<uint8>&& InNewData) override
	{
		FScopeLock lock(&Lock);
		NumBytesAdded += InNewData.Num();
		Buffer.Append(MoveTemp(InNewData));
	}

	virtual void AddData(const TConstArrayView<const uint8>& InNewData) override
	{
		FScopeLock lock(&Lock);
		NumBytesAdded += InNewData.Num();
		Buffer.Append(InNewData.GetData(), InNewData.Num());
	}

	virtual void AddData(const IElectraHTTPStreamBuffer& InOther, int64 Offset, int64 NumBytes) override
	{
		const FElectraHTTPStreamBuffer& Other = static_cast<const FElectraHTTPStreamBuffer&>(InOther);
		FScopeLock lockOther(&Other.Lock);
		check(Other.IsCachable());
		check(Offset < Other.GetNumTotalBytesAdded());
		check(Offset+NumBytes <= Other.GetNumTotalBytesAdded());
		if (Other.IsCachable() && Offset < Other.GetNumTotalBytesAdded() && Offset+NumBytes <= Other.GetNumTotalBytesAdded())
		{
			FScopeLock lock(&Lock);
			Buffer.Append(Other.Buffer.GetData() + Offset, NumBytes);
			NumBytesAdded += NumBytes;
		}
	}

	virtual int64 GetNumTotalBytesAdded() const override
	{
		return NumBytesAdded;
	}

	virtual int64 GetNumTotalBytesHandedOut() const override
	{
		return NumBytesHandedOut;
	}

	virtual int64 GetNumBytesAvailableForRead()	const override
	{
		FScopeLock lock(&Lock);
		int64 NumInBuffer = Buffer.Num();
		return NumInBuffer - NextReadPosInBuffer;
	}

	virtual void LockBuffer(const uint8*& OutNextReadAddress, int64& OutNumBytesAvailable) override
	{
		Lock.Lock();
		// If the buffer is already locked by the current thread this tends to be an indication of
		// a LockBuffer() / UnlockBuffer() mismatch.
		check(!bIsBufferLocked);
		bIsBufferLocked = true;
		OutNextReadAddress = Buffer.GetData() + NextReadPosInBuffer;
		OutNumBytesAvailable = Buffer.Num() - NextReadPosInBuffer;
	}
	virtual void UnlockBuffer(int64 NumBytesConsumed) override
	{
		check(NumBytesConsumed >= 0 && NumBytesConsumed <= Buffer.Num() - NextReadPosInBuffer);
		if (NumBytesConsumed > 0)
		{
			if (NumBytesConsumed > Buffer.Num() - NextReadPosInBuffer)
			{
				NumBytesConsumed = Buffer.Num() - NextReadPosInBuffer;
			}
			NextReadPosInBuffer += NumBytesConsumed;
			// Note: We could pop off the amount consumed and shrink the buffer now if necessary.
			//       The cost of the memmove may be too large though and the response will then
			//		 also not be cachable any more!
			// bIsCachable = false;

			// Update the total amount of bytes handed out.
			NumBytesHandedOut += NumBytesConsumed;
		}
		bIsBufferLocked = false;
		Lock.Unlock();
	}

	virtual bool RewindToBeginning() override
	{
		if (bIsCachable)
		{
			NextReadPosInBuffer = 0;
			NumBytesHandedOut = 0;
			return true;
		}
		return false;
	}

	virtual bool GetEOS() const override
	{ return bEOSReceived; }
	virtual void SetEOS() override
	{ bEOSReceived = true; }
	virtual void ClearEOS() override
	{ bEOSReceived = false; }

	virtual bool HasAllDataBeenConsumed() const override
	{ return GetEOS() && GetNumBytesAvailableForRead() == 0; }

	virtual bool IsCachable() const override
	{ return bIsCachable; }
	virtual void SetIsCachable(bool bInIsCachable) override
	{ bIsCachable = bInIsCachable; }

	virtual void SetLengthFromResponseHeader(int64 InLengthFromResponseHeader) override
	{ 
		FScopeLock lock(&Lock);
		LengthFromResponseHeader = InLengthFromResponseHeader; 
		Buffer.Reserve(LengthFromResponseHeader);
	}
	virtual int64 GetLengthFromResponseHeader() const override
	{ return LengthFromResponseHeader; }

protected:
	virtual void GetBaseBuffer(const uint8*& OutBaseAddress, int64& OutBytesInBuffer) override
	{
		FScopeLock lock(&Lock);
		OutBaseAddress = Buffer.GetData();
		OutBytesInBuffer = Buffer.Num();
	}

private:
	mutable FCriticalSection Lock;
	TArray<uint8> Buffer;
	int64 NextReadPosInBuffer = 0;
	bool bIsBufferLocked = false;
	// Total length as specified by Content-Length or Content-Range header.
	// Could be -1 if either is absent when chunked transfer encoding is in effect.
	// It is PERMITTED for this value to be LESS than the actual data since it is
	// the size of the compressed data, not the uncompressed!
	// This will also not be set for HEAD requests since no data will be downloaded.
	std::atomic_int64_t LengthFromResponseHeader { -1 };
	// End-of-Stream received. No additional data will be added.
	bool bEOSReceived = false;
	// Number of bytes added to the buffer so far with calls to AddData().
	int64 NumBytesAdded = 0;
	// Number of bytes handed out.
	int64 NumBytesHandedOut = 0;
	// An indicator if the entire buffer data is still available or not. If the buffer
	// is being truncated while read then the buffer cannot be cached any more.
	bool bIsCachable = true;
};
