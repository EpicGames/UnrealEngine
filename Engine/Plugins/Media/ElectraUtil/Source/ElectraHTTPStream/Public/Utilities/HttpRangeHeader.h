// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>

namespace ElectraHTTPStream
{

	struct FHttpRange
	{
		void Reset()
		{
			Start = -1;
			EndIncluding = -1;
			DocumentSize = -1;
		}
		bool IsSet() const
		{
			return Start != -1 || EndIncluding != -1;
		}
		bool Equals(const FHttpRange& Other)
		{
			return Start == Other.Start && EndIncluding == Other.EndIncluding;
		}
		// Check if the range would result in "0-" for the entire resource in which case we don't need to use range request.
		bool IsEverything() const
		{
			return Start <= 0 && EndIncluding < 0;
		}
		FString GetString(bool bWithUnitPrefix=false) const
		{
			FString Hdr = bWithUnitPrefix ? TEXT("bytes=") : TEXT("");
			FString s, e, d(TEXT("-"));
			if (Start >= 0)
			{
				// An explicit range?
				if (EndIncluding >= Start)
				{
					Hdr.Append(FString::Printf(TEXT("%lld-%lld"), (long long int)Start, (long long int)EndIncluding));
					return Hdr;
				}
				// Start to end, whereever that is.
				else
				{
					Hdr.Append(FString::Printf(TEXT("%lld-"), (long long int)Start));
					return Hdr;
				}
			}
			// Everything
			Hdr.Append(TEXT("0-"));
			return Hdr;
		}
		// Returns the number of bytes in the range, which must be fully specified. An unset or partially open range will return -1.
		int64 GetNumberOfBytes() const
		{
			if (Start >= 0 && EndIncluding >= 0)
			{
				return EndIncluding - Start + 1;
			}
			return -1;
		}
		int64 GetStart() const
		{
			return Start;
		}
		int64 GetEndIncluding() const
		{
			return EndIncluding;
		}
		bool IsOpenEnded() const
		{
			return GetEndIncluding() < 0;
		}
		void Set(const FString& InString)
		{
			int32 DashPos = INDEX_NONE;
			if (InString.FindChar(TCHAR('-'), DashPos))
			{
				// -end
				if (DashPos == 0)
				{
					Start = 0;
					LexFromString(EndIncluding, *InString + 1);
				}
				// start-
				else if (DashPos == InString.Len()-1)
				{
					LexFromString(Start, *InString.Mid(0, DashPos));
					EndIncluding = -1;
				}
				// start-end
				else
				{
					LexFromString(Start, *InString.Mid(0, DashPos));
					LexFromString(EndIncluding, *InString + DashPos + 1);
				}
			}
		}
		void SetStart(int64 InStart)
		{
			Start = InStart;
		}
		void SetEndIncluding(int64 InEndIncluding)
		{
			EndIncluding = InEndIncluding;
		}
		int64 GetDocumentSize() const
		{
			return DocumentSize;
		}
		bool ParseFromContentRangeResponse(const FString& ContentRangeHeader)
		{
			// Examples: <unit> <range-start>-<range-end>/<size>
			//   Content-Range: bytes 26151-157222/7594984
			//   Content-Range: bytes 26151-157222/*
			//   Content-Range: bytes */7594984
			//
			Start = -1;
			EndIncluding = -1;
			DocumentSize = -1;
			FString rh = ContentRangeHeader;
			// In case the entire header is given, remove the header including the separating colon and space.
			rh.RemoveFromStart(TEXT("Content-Range: "), ESearchCase::CaseSensitive);
			// Split into parts
			TArray<FString> Parts;
			const TCHAR* const Delims[3] = {TEXT(" "),TEXT("-"),TEXT("/")};
			int32 NumParts = rh.ParseIntoArray(Parts, Delims, 3);
			if (NumParts)
			{
				if (Parts[0] == TEXT("bytes"))
				{
					Parts.RemoveAt(0);
				}
				// We should now be left with 3 remaining results, the start, end and document size.
				if (Parts.Num() == 3)
				{
					if (Parts[0].IsNumeric())
					{
						LexFromString(Start, *Parts[0]);
						if (Parts[1].IsNumeric())
						{
							LexFromString(EndIncluding, *Parts[1]);
							if (Parts[2].IsNumeric())
							{
								LexFromString(DocumentSize, *Parts[2]);
								return true;
							}
							else if (Parts[2] == TEXT("*"))
							{
								DocumentSize = -1;
								return true;
							}
						}
					}
				}
				// The case where we get "*/<size>" we treat as invalid.
				else if (Parts.Num() == 2)
				{
					return false;
				}
			}
			return false;
		}
		int64			Start = -1;
		int64			EndIncluding = -1;
		int64			DocumentSize = -1;
	};


}

