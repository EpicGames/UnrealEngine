// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;
using System.IO;
using EpicGames.Core;

namespace UnrealBuildTool.Rules
{
	public class ElectraSamples: ModuleRules
	{
		public ElectraSamples(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			bLegalToDistributeObjectCode = true;

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"MediaUtils",
					"RenderCore",
					"RHI",
					"ElectraBase",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Media",
				});

			if (Target.bCompileAgainstEngine)
			{
				PrivateDependencyModuleNames.Add("Engine");
			}

			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				PublicDependencyModuleNames.Add("DirectX");

				AddEngineThirdPartyPrivateStaticDependencies(Target, "DX9");

				if (Target.WindowsPlatform.Architecture != UnrealArch.Arm64)
				{
					PublicAdditionalLibraries.AddRange(new string[] {
						Path.Combine(Target.WindowsPlatform.DirectXLibDir, "dxerr.lib"),
					});
				}

				PrivateDependencyModuleNames.AddAll("D3D11RHI", "D3D12RHI");
				AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11", "DX12");

				PrivateDefinitions.Add("ELECTRA_SUPPORT_PREWIN8");
				PrivateDefinitions.Add("ELECTRA_HAVE_DX11");

				PublicSystemLibraries.AddRange(new string[] {
					"strmiids.lib",
					"legacy_stdio_definitions.lib",
					"Dxva2.lib",
				});

				PublicIncludePaths.Add("$(ModuleDir)/Public/Windows");
			}
			else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Apple))
			{
				PrivateDependencyModuleNames.Add("MetalRHI");
				AddEngineThirdPartyPrivateStaticDependencies(Target, "MetalCPP");
				PublicIncludePaths.Add("$(ModuleDir)/Public/Apple");
			}
			else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Android))
			{
				PublicIncludePaths.Add("$(ModuleDir)/Public/Android");
				string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
				AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "ElectraSamples_UPL.xml"));
				PrivateDependencyModuleNames.Add("VulkanRHI");
				AddEngineThirdPartyPrivateStaticDependencies(Target, "Vulkan");
			}
			else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
			{
				PublicIncludePaths.Add("$(ModuleDir)/Public/Linux");
			}
		}
	}
}
