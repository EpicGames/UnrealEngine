// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

#if PLATFORM_ANDROID

#include "ElectraTextureSample.h"
#include "ElectraSamplesModule.h"

#include "Android/AndroidPlatform.h"
#include "Android/AndroidJava.h"
#include "Android/AndroidApplication.h"

#include "Containers/AnsiString.h"

#include "RenderingThread.h"
#include "RHIStaticStates.h"
#include "PipelineStateCache.h"

#include "RenderUtils.h"

#include "VulkanCommon.h"
#include "IVulkanDynamicRHI.h"

#include <EGL/egl.h>
#include <EGL/eglext.h>
#include <GLES3/gl31.h>
#include <GLES2/gl2ext.h>
#include <android/hardware_buffer.h>
#include <android/hardware_buffer_jni.h>
#include <vulkan/vulkan_android.h>

DECLARE_GPU_STAT_NAMED(MediaAndroidDecoder_Convert, TEXT("MediaAndroidDecoder_Convert"));

/*********************************************************************************************************************/

static TAutoConsoleVariable<int32> CVarElectraAndroidUseGpuOutputPath(
	TEXT("Electra.AndroidUseGpuOutputPath"),
	0,
	TEXT("Use experimental direct to GPU output path on Android.\n")
	TEXT(" 0: use CPU output path (default); 1: use new direct to GPU output path."),
	ECVF_Default);

/*********************************************************************************************************************/

#define ELECTRA_INIT_ON_RENDERTHREAD	1	// set to 1 if context & surface init should be on render thread (seems safer for compatibility - TODO: research why)

// ---------------------------------------------------------------------------------------------------------------------

#if UE_BUILD_SHIPPING
// always clear any exceptions in shipping
#define CHECK_JNI_RESULT(Id) if (Id == 0) { JEnv->ExceptionClear(); }
#else
#define CHECK_JNI_RESULT(Id)								\
	if (Id == 0)											\
	{														\
		if (!bIsOptional)									\
		{													\
			JEnv->ExceptionDescribe();						\
			checkf(Id != 0, TEXT("Failed to find " #Id));	\
		}													\
		JEnv->ExceptionClear();								\
	}
#endif

// ---------------------------------------------------------------------------------------------------------------------

namespace
{

static void CleanupImageResourcesJNI(jobject Resources, jmethodID ReleaseFN)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FRHICommandReleaseDecoderResources_Execute);

	JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
	JEnv->CallVoidMethod(Resources, ReleaseFN);
	JEnv->DeleteGlobalRef(Resources);
}

} // namespace

namespace UE::Jni
{
	struct FElectraTextureSample: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/ElectraTextureSample";

		inline static TConstructor<FElectraTextureSample()> New;

		static void JNICALL nativeSignalSurfaceReadEvent(JNIEnv* env, jobject thiz, jlong InParentHandle);

		static constexpr FMember Members[]
		{
			UE_JNI_MEMBER(New)
		};

		static constexpr FNativeMethod NativeMethods[]
		{
			UE_JNI_NATIVE_METHOD(nativeSignalSurfaceReadEvent)
		};
	};

	template struct TInitialize<FElectraTextureSample>;
}

// ---------------------------------------------------------------------------------------------------------------------

class FElectraTextureSampleSupport : public UE::Jni::TClassObject<UE::Jni::FElectraTextureSample>
{
public:
	FElectraTextureSampleSupport();
	~FElectraTextureSampleSupport();

	int32 GetFrameData(FElectraTextureSample* InTargetSample);
	jobject GetCodecSurface();

	jmethodID GetImageResources_ReleaseFN() const
	{
		return FImageResources_ReleaseFN;
	}

	void SignalImageReaderSurfaceRead()
	{
		if (CodecSurfaceReadEvent)
		{
			CodecSurfaceReadEvent->Trigger();
		}
	}

	jobject ImageResources_GetHardwareBuffer(jobject ImageResources);
	void ImageResources_GetScaleOffset(jobject ImageResources, FVector2f& OutScale, FVector2f& OutOffset);

	bool UseGpuOutputPath() const
	{
		return bUseGpuOutputPath;
	}

private:
	bool bUseGpuOutputPath;

	// Java methods
	FJavaClassMethod	InitializeFN;
	FJavaClassMethod	ReleaseFN;
	FJavaClassMethod	GetCodecSurfaceFN;
	FJavaClassMethod	GetVideoFrameUpdateInfoFN;

	// FFrameUpdateInfo member field IDs
	jclass				FFrameUpdateInfoClass;
	jfieldID			FFrameUpdateInfo_Buffer;
	jfieldID			FFrameUpdateInfo_Timestamp;
	jfieldID			FFrameUpdateInfo_Duration;
	jfieldID			FFrameUpdateInfo_bFrameReady;
	jfieldID			FFrameUpdateInfo_bRegionChanged;
	jfieldID			FFrameUpdateInfo_UScale;
	jfieldID			FFrameUpdateInfo_UOffset;
	jfieldID			FFrameUpdateInfo_VScale;
	jfieldID			FFrameUpdateInfo_VOffset;
	jfieldID			FFrameUpdateInfo_NumPending;
	jfieldID			FFrameUpdateInfo_ImageResources;

	// FImageResources members / methods
	jclass				FImageResourcesClass;
	jfieldID			FImageResources_HardwareBufferHandle;
	jfieldID			FImageResources_UScale;
	jfieldID			FImageResources_VScale;
	jfieldID			FImageResources_UOffset;
	jfieldID			FImageResources_VOffset;
	jmethodID			FImageResources_ReleaseFN;

	jobject				CodecSurface;
	FEvent*				SurfaceInitEvent;
	FCriticalSection	CodecSurfaceLock;
	jobject				CodecSurfaceToDelete;
	FEvent*				CodecSurfaceReadEvent;

	static FName GetClassName()
	{
		return FName("com/epicgames/unreal/ElectraTextureSample");
	}

	jfieldID FindField(JNIEnv* JEnv, jclass InClass, const ANSICHAR* InFieldName, const ANSICHAR* InFieldType, bool bIsOptional)
	{
		jfieldID Field = InClass == nullptr ? nullptr : JEnv->GetFieldID(InClass, InFieldName, InFieldType);
		CHECK_JNI_RESULT(Field);
		return Field;
	}

	jmethodID FindMethod(JNIEnv* JEnv, jclass InClass, const ANSICHAR* InMethodName, const ANSICHAR* InMethodSignature, bool bIsOptional)
	{
		jmethodID Method = InClass == nullptr ? nullptr : JEnv->GetMethodID(InClass, InMethodName, InMethodSignature);
		CHECK_JNI_RESULT(Method);
		return Method;
	}

	// Create a Java byte array. Must DeleteLocalRef() after use or handling over to Java.
	jbyteArray MakeJavaByteArray(const uint8* InData, int32 InNumBytes)
	{
		JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
		jbyteArray RawBuffer = JEnv->NewByteArray(InNumBytes);
		JEnv->SetByteArrayRegion(RawBuffer, 0, InNumBytes, reinterpret_cast<const jbyte*>(InData));
		return RawBuffer;
	}

};

// ---------------------------------------------------------------------------------------------------------------------

FElectraTextureSampleSupport::FElectraTextureSampleSupport()
	: bUseGpuOutputPath(false)
	, InitializeFN(GetClassMethod("Initialize", "(ZZJ)V"))
	, ReleaseFN(GetClassMethod("Release", "()V"))
	, GetCodecSurfaceFN(GetClassMethod("GetCodecSurface", "()Landroid/view/Surface;"))
	, GetVideoFrameUpdateInfoFN(GetClassMethod("GetVideoFrameUpdateInfo", "(IIIZ)Lcom/epicgames/unreal/ElectraTextureSample$FFrameUpdateInfo;"))
	, CodecSurface(nullptr)
	, SurfaceInitEvent(nullptr)
	, CodecSurfaceToDelete(nullptr)
	, CodecSurfaceReadEvent(nullptr)
{
	if (FAndroidMisc::ShouldUseVulkan()) // GpuOutputPath is only available for Vulkan right now (and experimental)
	{
		if (CVarElectraAndroidUseGpuOutputPath.GetValueOnAnyThread() == 0)
		{
			UE_LOGF(LogElectraSamples, Log, "Selecting CPU path because GPU path is disabled via Electra.AndroidUseGpuOutputPath = 0");
		}
		else
		{
			IVulkanDynamicRHI* RHI = GetIVulkanDynamicRHI();
			TArray<FAnsiString> LoadedDeviceExtensions = RHI->RHIGetLoadedDeviceExtensions();
			if (!LoadedDeviceExtensions.Find(FAnsiString(VK_KHR_SAMPLER_YCBCR_CONVERSION_EXTENSION_NAME)))
			{
				UE_LOG(LogElectraSamples, Log, TEXT("Selecting CPU path because GPU extension '" VK_KHR_SAMPLER_YCBCR_CONVERSION_EXTENSION_NAME "' is not available!"));
			}
			else if (!LoadedDeviceExtensions.Find(FAnsiString(VK_ANDROID_EXTERNAL_MEMORY_ANDROID_HARDWARE_BUFFER_EXTENSION_NAME)))
			{
				UE_LOG(LogElectraSamples, Log, TEXT("Selecting CPU path because GPU extension '" VK_ANDROID_EXTERNAL_MEMORY_ANDROID_HARDWARE_BUFFER_EXTENSION_NAME "' is not available!"));
			}
			else
			{
				bUseGpuOutputPath = true;
				UE_LOGF(LogElectraSamples, Log, "Selecting GPU path because it is enabled via Electra.AndroidUseGpuOutputPath = 1");
			}
		}
	}
	else
	{
		UE_LOGF(LogElectraSamples, Log, "Selecting CPU path because we are on OES");
	}

	JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();

	// Get field IDs for FFrameUpdateInfo class members
	jclass localFrameUpdateInfoClass = FAndroidApplication::FindJavaClass("com/epicgames/unreal/ElectraTextureSample$FFrameUpdateInfo");
	FFrameUpdateInfoClass = (jclass)JEnv->NewGlobalRef(localFrameUpdateInfoClass);
	JEnv->DeleteLocalRef(localFrameUpdateInfoClass);
	FFrameUpdateInfo_Buffer = FindField(JEnv, FFrameUpdateInfoClass, "Buffer", "Ljava/nio/Buffer;", false);
	FFrameUpdateInfo_Timestamp = FindField(JEnv, FFrameUpdateInfoClass, "Timestamp", "J", false);
	FFrameUpdateInfo_Duration = FindField(JEnv, FFrameUpdateInfoClass, "Duration", "J", false);
	FFrameUpdateInfo_bFrameReady = FindField(JEnv, FFrameUpdateInfoClass, "bFrameReady", "Z", false);
	FFrameUpdateInfo_bRegionChanged = FindField(JEnv, FFrameUpdateInfoClass, "bRegionChanged", "Z", false);
	FFrameUpdateInfo_UScale = FindField(JEnv, FFrameUpdateInfoClass, "UScale", "F", false);
	FFrameUpdateInfo_UOffset = FindField(JEnv, FFrameUpdateInfoClass, "UOffset", "F", false);
	FFrameUpdateInfo_VScale = FindField(JEnv, FFrameUpdateInfoClass, "VScale", "F", false);
	FFrameUpdateInfo_VOffset = FindField(JEnv, FFrameUpdateInfoClass, "VOffset", "F", false);
	FFrameUpdateInfo_NumPending = FindField(JEnv, FFrameUpdateInfoClass, "NumPending", "I", false);
	FFrameUpdateInfo_ImageResources = FindField(JEnv, FFrameUpdateInfoClass, "ImageResources", "Lcom/epicgames/unreal/ElectraTextureSample$FImageResources;", false);

	// Get field IDs for FImageResources class members (etc.)
	jclass localImageResourcesClass = FAndroidApplication::FindJavaClass("com/epicgames/unreal/ElectraTextureSample$FImageResources");
	FImageResourcesClass = (jclass)JEnv->NewGlobalRef(localImageResourcesClass);
	JEnv->DeleteLocalRef(localImageResourcesClass);
	FImageResources_HardwareBufferHandle = FindField(JEnv, FImageResourcesClass, "HardwareBuffer", "Landroid/hardware/HardwareBuffer;", false);
	FImageResources_UScale = FindField(JEnv, FImageResourcesClass, "UScale", "F", false);
	FImageResources_UOffset = FindField(JEnv, FImageResourcesClass, "UOffset", "F", false);
	FImageResources_VScale = FindField(JEnv, FImageResourcesClass, "VScale", "F", false);
	FImageResources_VOffset = FindField(JEnv, FImageResourcesClass, "VOffset", "F", false);

	FImageResources_ReleaseFN = FindMethod(JEnv, FImageResourcesClass, "Release", "()V", false);

	// - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - -

	int32 SDKint = 0;
	jclass localVersionClass = JEnv->FindClass("android/os/Build$VERSION");
	if (localVersionClass)
	{
		jfieldID SdkIntFieldID = JEnv->GetStaticFieldID(localVersionClass, "SDK_INT", "I");
		if (SdkIntFieldID)
		{
			SDKint = JEnv->GetStaticIntField(localVersionClass, SdkIntFieldID);
		}
		else
		{
			JEnv->ExceptionClear();
		}
		JEnv->DeleteLocalRef(localVersionClass);
	}
	else
	{
		JEnv->ExceptionClear();
	}

	// - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - - -

	// Does the SDK support for KEY_ALLOW_FRAME_DROP in MediaFormat exist (and hence allow for throttle-free use of the Surface queue)?
	if (SDKint < 31)
	{
		// No. Setup "read from surface" event to allow throttling
		CodecSurfaceReadEvent = FPlatformProcess::GetSynchEventFromPool();
		ensure(CodecSurfaceReadEvent != nullptr);
		CodecSurfaceReadEvent->Trigger();
	}
	else
	{
		// Yes! No need for throttling...
		CodecSurfaceReadEvent = nullptr;
	}

#if ELECTRA_INIT_ON_RENDERTHREAD
	// enqueue to RT to ensure GL resources are created on the appropriate thread.
	SurfaceInitEvent = FPlatformProcess::GetSynchEventFromPool(true);
	ENQUEUE_RENDER_COMMAND(InitElectraTextureSample)([this](FRHICommandListImmediate& RHICmdList)
		{
			RHICmdList.EnqueueLambda([this](FRHICommandListImmediate& CmdList)
				{
					JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
					// Setup Java side of things
					JEnv->CallVoidMethod(Object, InitializeFN.Method, UseGpuOutputPath(), FAndroidMisc::ShouldUseVulkan(), (jlong)this);
					// Query surface to be used for decoder
					jobject Surface = JEnv->CallObjectMethod(Object, GetCodecSurfaceFN.Method);
					CodecSurface = JEnv->NewGlobalRef(Surface);
					JEnv->DeleteLocalRef(Surface);
					SurfaceInitEvent->Trigger();
				});
		});
	FlushRenderingCommands();
#else
	// Setup Java side of things
	JEnv->CallVoidMethod(Object, InitializeFN.Method, UseGpuOutputPath(), FAndroidMisc::ShouldUseVulkan(), (jlong)this);

	// Query surface to be used for decoder
	jobject Surface = JEnv->CallObjectMethod(Object, GetCodecSurfaceFN.Method);
	CodecSurface = JEnv->NewGlobalRef(Surface);
	JEnv->DeleteLocalRef(Surface);
#endif
}


FElectraTextureSampleSupport::~FElectraTextureSampleSupport()
{
	// When initialization of the surface was triggered on the rendering thread we need to wait for its completion.
	if (SurfaceInitEvent)
	{
		// Wait for the surface initialization event to have been signaled.
		// Do not wait if we are on the render thread. In this case the initialization has already completed anyway.
		if (!IsInRenderingThread())
		{
			SurfaceInitEvent->Wait();
		}
		FPlatformProcess::ReturnSynchEventToPool(SurfaceInitEvent);
		SurfaceInitEvent = nullptr;
	}

	CodecSurfaceLock.Lock();
	CodecSurfaceToDelete = CodecSurface;
	CodecSurface = nullptr;
	CodecSurfaceLock.Unlock();

	if (CodecSurfaceReadEvent)
	{
		FPlatformProcess::ReturnSynchEventToPool(CodecSurfaceReadEvent);
		CodecSurfaceReadEvent = nullptr;
	}

	if (IsInGameThread())
	{
		// enqueue to RT to ensure GL resources are released on the appropriate thread.
		ENQUEUE_RENDER_COMMAND(DestroyElectraTextureSample)([this](FRHICommandListImmediate& RHICmdList)
		{
			RHICmdList.EnqueueLambda([this](FRHICommandListImmediate& CmdList)
			{
				JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
				JEnv->CallVoidMethod(Object, ReleaseFN.Method);
				if (CodecSurfaceToDelete)
				{
					JEnv->DeleteGlobalRef(CodecSurfaceToDelete);
				}
				JEnv->DeleteGlobalRef(FImageResourcesClass);
				JEnv->DeleteGlobalRef(FFrameUpdateInfoClass);
				CodecSurfaceToDelete = nullptr;
				FImageResourcesClass = nullptr;
				FFrameUpdateInfoClass = nullptr;
			});
		});
		FlushRenderingCommands();
	}
	else
	{
		JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
		JEnv->CallVoidMethod(Object, ReleaseFN.Method);
		if (CodecSurfaceToDelete)
		{
			JEnv->DeleteGlobalRef(CodecSurfaceToDelete);
		}
		JEnv->DeleteGlobalRef(FImageResourcesClass);
		JEnv->DeleteGlobalRef(FFrameUpdateInfoClass);
		CodecSurfaceToDelete = nullptr;
		FImageResourcesClass = nullptr;
		FFrameUpdateInfoClass = nullptr;
	}
}

//-----------------------------------------------------------------------------
/**
 *
 * @note Call this from an RHI thread! It will need a valid rendering environment!
 */
int32 FElectraTextureSampleSupport::GetFrameData(FElectraTextureSample* InTargetSample)
{
	// In case this is called with a ES renderer, we need to pass in the destination texture we'd like to be used to receive the data
	// (for Vulkan we'll just receive a simple byte buffer)
	int32 DestTexture = 0;
	if (FRHITexture* Texture = InTargetSample->GetTexture())
	{
		DestTexture = *reinterpret_cast<int32*>(Texture->GetNativeResource());
	}

	// Update frame info and get data...
	JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
	jobject OutputInfo = JEnv->CallObjectMethod(Object, GetVideoFrameUpdateInfoFN.Method, DestTexture, InTargetSample->GetDim().X, InTargetSample->GetDim().Y, InTargetSample->GetFormat() == EMediaTextureSampleFormat::CharBGR10A2);
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
	}
	// Failure will return no object.
	if (OutputInfo != nullptr)
	{
		if (InTargetSample)
		{
			if (UseGpuOutputPath())
			{
				jobject ImageResources = JEnv->GetObjectField(OutputInfo, FFrameUpdateInfo_ImageResources);
				if (ImageResources)
				{
					InTargetSample->SetImageResources(ImageResources);
					JEnv->DeleteLocalRef(ImageResources);
				}
			}
			else
			{
				jobject buffer = JEnv->GetObjectField(OutputInfo, FFrameUpdateInfo_Buffer);
				if (buffer != nullptr)
				{
					const void* outPixels = JEnv->GetDirectBufferAddress(buffer);
					const int32 outCount = JEnv->GetDirectBufferCapacity(buffer);
					InTargetSample->SetupFromBuffer(outPixels, outCount);
					JEnv->DeleteLocalRef(buffer);
				}
			}
		}

		JEnv->DeleteLocalRef(OutputInfo);
		return 0;
	}
	return 1;
}


jobject FElectraTextureSampleSupport::GetCodecSurface()
{
	if (SurfaceInitEvent)
	{
		// Wait for the surface initialization event to have been signaled.
		// Do not wait if we are on the render thread. In this case the initialization has already completed anyway.
		if (!IsInRenderingThread())
		{
			// Only wait for a little while here just in case this would prevent the render thread from
			// even starting its jobs and us causing a deadlock here.
			bool bInitDone = SurfaceInitEvent->Wait(FTimespan::FromMilliseconds(2000.0));
			if (bInitDone)
			{
				// When init has completed we can free the event and do not have to wait for it any more in the future.
				FPlatformProcess::ReturnSynchEventToPool(SurfaceInitEvent);
				SurfaceInitEvent = nullptr;
			}
		}
	}

	// Create a new global ref to return.
	jobject NewSurfaceHandle = nullptr;
	CodecSurfaceLock.Lock();
	if (CodecSurface)
	{
		JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
		NewSurfaceHandle = JEnv->NewGlobalRef(CodecSurface);
	}
	CodecSurfaceLock.Unlock();
	return NewSurfaceHandle;
}

jobject FElectraTextureSampleSupport::ImageResources_GetHardwareBuffer(jobject ImageResources)
{
	JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
	return JEnv->GetObjectField(ImageResources, FImageResources_HardwareBufferHandle);
}


void FElectraTextureSampleSupport::ImageResources_GetScaleOffset(jobject ImageResources, FVector2f& OutScale, FVector2f& OutOffset)
{
	JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
	OutScale.X = JEnv->GetFloatField(ImageResources, FImageResources_UScale);
	OutScale.Y = JEnv->GetFloatField(ImageResources, FImageResources_VScale);
	OutOffset.X = JEnv->GetFloatField(ImageResources, FImageResources_UOffset);
	OutOffset.Y = JEnv->GetFloatField(ImageResources, FImageResources_VOffset);
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------

void JNICALL UE::Jni::FElectraTextureSample::nativeSignalSurfaceReadEvent(JNIEnv* jenv, jobject thiz, jlong InParentHandle)
{
	auto Instance = reinterpret_cast<FElectraTextureSampleSupport*>(InParentHandle);
	if (Instance)
	{
		Instance->SignalImageReaderSurfaceRead();
	}
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------

FElectraTextureSamplePool::FElectraTextureSamplePool()
	: TMediaObjectPool<TextureSample, FElectraTextureSamplePool>(this)
	, Support(new FElectraTextureSampleSupport())
{}


void* FElectraTextureSamplePool::GetCodecSurface()
{
	return (void*)Support->GetCodecSurface();
}

//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------
//-----------------------------------------------------------------------------



FElectraTextureSample::FElectraTextureSample(TSharedPtr<FElectraTextureSampleSupport, ESPMode::ThreadSafe> InSupport)
	: Support(InSupport)
	, ImageResources(nullptr)
	, Texture(nullptr)
	, Buffer(nullptr)
	, BufferSize(0)
{
}

bool FElectraTextureSample::FinishInitialization()
{
	return IElectraTextureSampleBase::FinishInitialization();
}


bool FElectraTextureSample::Initialize()
{
	if (Support->UseGpuOutputPath())
	{
		bQueuedForConversion = false;
		Support->GetFrameData(this);
		Texture = nullptr;
	}
	else
	{
		ENQUEUE_RENDER_COMMAND(InitTextureSample)([WeakThis{ AsWeak() }](FRHICommandListImmediate& RHICmdList) {
			if (TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe> This = WeakThis.Pin())
			{
				This->InitializeTexture(RHICmdList, This->GetPixelFormat());

				if (This->Texture)
				{
					RHICmdList.EnqueueLambda([WeakThis](FRHICommandListImmediate& CmdList)
					{
						QUICK_SCOPE_CYCLE_COUNTER(STAT_FRHICommandUpdateDecoderExternaTexture_Execute);
						if (TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe> This = WeakThis.Pin())
						{
							This->Support->GetFrameData(This.Get());
						}
					});
				}
				else
				{
					// CPU side buffer case: we do not need to do that on an RHI thread
					This->Support->GetFrameData(This.Get());
				}
			}
		});
	}
	return true;
}

FTextureRHIRef FElectraTextureSample::InitializeTextureOES(AHardwareBuffer* HardwareBuffer)
{
	check(HardwareBuffer);
	check(IsInRenderingThread() || IsInRHIThread());
	check(!FAndroidMisc::ShouldUseVulkan());

#if 0 // TODO this needs to run on the OpenGlContext thread, so it probably will have to go into RHI before it can work

	RHICmdList.EnqueueLambda([WeakThis{ AsWeak() }, &InDstTexture](FRHICommandListImmediate& CmdList)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FRHICommandUpdateDecoderExternaTexture_Execute);
		if (TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe> This = WeakThis.Pin())
		{
			if (This->ImageResources == nullptr)
			{
				This->Texture = nullptr;
				return;
			}

			jobject HardwareBufferObj = This->Support->ImageResources_GetHardwareBuffer(This->ImageResources);
			AHardwareBuffer* HardwareBuffer = AHardwareBuffer_fromHardwareBuffer(FAndroidApplication::GetJavaEnv(), HardwareBufferObj);

			static PFNEGLGETNATIVECLIENTBUFFERANDROIDPROC EglGetNativeClientBufferANDROID = nullptr;
			if (EglGetNativeClientBufferANDROID == nullptr)
			{
				EglGetNativeClientBufferANDROID = (PFNEGLGETNATIVECLIENTBUFFERANDROIDPROC)((void*)eglGetProcAddress("eglGetNativeClientBufferANDROID"));
				ensure(EglGetNativeClientBufferANDROID != nullptr);
			}

			// Ext. EGL_ANDROID_get_native_client_buffer
			EGLClientBuffer NativeClientBuffer = EglGetNativeClientBufferANDROID(HardwareBuffer);
			if (NativeClientBuffer == nullptr)
			{
				UE_LOGF(LogElectraSamples, Warning, "Could not get native client buffer!");
				return;
			}

			static PFNEGLCREATEIMAGEKHRPROC EglCreateImageKHR = nullptr;
			if (EglCreateImageKHR == nullptr)
			{
				EglCreateImageKHR = (PFNEGLCREATEIMAGEKHRPROC)((void*)eglGetProcAddress("eglCreateImageKHR"));
				ensure(EglCreateImageKHR != nullptr);
			}

			// Ext. EGL_ANDROID_image_native_buffer
			EGLImageKHR EglImage = EglCreateImageKHR(eglGetCurrentDisplay(), EGL_NO_CONTEXT, EGL_NATIVE_BUFFER_ANDROID, NativeClientBuffer, nullptr);
			if (EglImage == 0)
			{
				UE_LOGF(LogElectraSamples, Warning, "Could not create EGLimage from native client buffer! B=0x%x E=0x%x", NativeClientBuffer, eglGetError());
				return;
			}

			static PFNGLEGLIMAGETARGETTEXTURE2DOESPROC GlEGLImageTargetTexture2DOES = nullptr;
			if (GlEGLImageTargetTexture2DOES == nullptr)
			{
				GlEGLImageTargetTexture2DOES = (PFNGLEGLIMAGETARGETTEXTURE2DOESPROC)((void*)eglGetProcAddress("glEGLImageTargetTexture2DOES"));
				ensure(GlEGLImageTargetTexture2DOES != nullptr);
			}

			GlEGLImageTargetTexture2DOES(GL_TEXTURE_EXTERNAL_OES, EglImage);
			check(eglGetError() == EGL_SUCCESS);
			check(glGetError() == 0);

			// Bind the RHI texture's GL texture to the external image we got data about now...
			glBindTexture(GL_TEXTURE_EXTERNAL_OES, *reinterpret_cast<int32*>(This->Texture->GetNativeResource()));
			check(glGetError() == 0);


		}
	});

	FIntPoint Dim = VideoDecoderOutput->GetDim();

	if (Texture.IsValid() && (Texture->GetSizeXY() == Dim || (Texture->GetFlags() & ETextureCreateFlags::External) == ETextureCreateFlags::External))
	{
		// The existing texture is just fine... (exists & same size OR external (size does not matter))
		return;
	}

	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(TEXT("FElectraTextureSampleExternalTexture"), 1, 1, PF_R8G8B8A8)
		.SetInitialState(ERHIAccess::SRVMask)
		.SetFlags(ETextureCreateFlags::External);

	return RHICmdList.CreateTexture(Desc);

#endif

	return nullptr;
}

FTextureRHIRef FElectraTextureSample::InitializeTextureVulkan(AHardwareBuffer* HardwareBuffer)
{
	check(HardwareBuffer);
	check(FAndroidMisc::ShouldUseVulkan());

	IVulkanDynamicRHI* RHI = GetIVulkanDynamicRHI();
	return RHI->RHICreateTexture2DFromAndroidHardwareBuffer(HardwareBuffer);
}


void FElectraTextureSample::InitializeTexture(FRHICommandListBase& RHICmdList, EPixelFormat PixelFormat)
{
	check(IsInRenderingThread() || IsInRHIThread());

	if (FAndroidMisc::ShouldUseVulkan())
	{
		// For Vulkan we use a CPU-side buffer to transport the data
		Texture = nullptr;
		return;
	}

	const FIntPoint Dim = GetDim();

	if (Texture.IsValid() && (Texture->GetSizeXY() == Dim))
	{
		// The existing texture is just fine...
		return;
	}

	// Make linear texture of appropriate bit depth to carry data...
	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(TEXT("FElectraTextureSample"))
		.SetExtent(Dim)
		.SetInitialState(ERHIAccess::SRVMask)
		.SetFlags(ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
		.SetFormat(PixelFormat);
	Texture = RHICmdList.CreateTexture(Desc);

	return;
}

void FElectraTextureSample::SetImageResources(jobject InImageResources)
{
	CleanupImageResources();
	ImageResources = FAndroidApplication::GetJavaEnv()->NewGlobalRef(InImageResources);
}


bool FElectraTextureSample::IsReadyForReuse()
{
	if (Support->UseGpuOutputPath())
	{
		if (bQueuedForConversion)
		{
			if (Fence->Poll())
			{
				bQueuedForConversion = false;
			}
		}

		return !bQueuedForConversion;
	}
	return true;
}

void FElectraTextureSample::ShutdownPoolable()
{
	if (bWasShutDown)
	{
		return;
	}
//Q: this will trigger us to reuse the RHI/GL texture over and over -> sounds good. But is it 100%?
//   - IF(!) the GL texture refs the EGL image internally we might keep the resources of that until a new one is assigned. Longer than needed! (although we free things up at the Java level (CleanupImageResources))
// 	 - MIGHT that starve the decoder for buffers (we only have a max. number of "Images", too)
//  >> set it to NULL and we'll recreate it... less ambigous maybe...
//	Texture = nullptr;

	if (Support->UseGpuOutputPath())
	{
		Texture = nullptr;
	}

	if (Fence)
	{
		Fence->Clear();
	}

	CleanupImageResources();

	IElectraTextureSampleBase::ShutdownPoolable();
}


void FElectraTextureSample::SetupFromBuffer(const void* InBuffer, int32 InBufferSize)
{
	if (BufferSize < InBufferSize)
	{
		if (BufferSize == 0)
		{
			Buffer = FMemory::Malloc(InBufferSize);
		}
		else
		{
			Buffer = FMemory::Realloc(Buffer, InBufferSize);
		}
		BufferSize = InBufferSize;
	}
	FMemory::Memcpy(Buffer, InBuffer, InBufferSize);
}

void FElectraTextureSample::CleanupImageResources()
{
	if (ImageResources)
	{
		jmethodID ReleaseFN = Support->GetImageResources_ReleaseFN();
		jobject Resources = ImageResources;

		if (Support->UseGpuOutputPath() || IsInRHIThread())
		{
			CleanupImageResourcesJNI(Resources, ReleaseFN);
		}
		else if (IsInRenderingThread())
		{
			FRHICommandListImmediate::Get().EnqueueLambda([Resources, ReleaseFN](FRHICommandListImmediate&)
			{
				CleanupImageResourcesJNI(Resources, ReleaseFN);
			});
		}
		else // Neither RHI nor Render thread
		{
			ENQUEUE_RENDER_COMMAND(ReleaseDecoderResources)([Resources, ReleaseFN](FRHICommandListImmediate& RHICmdList)
			{
				RHICmdList.EnqueueLambda([Resources, ReleaseFN](FRHICommandListImmediate&)
				{
					CleanupImageResourcesJNI(Resources, ReleaseFN);
				});
			});
		}

		ImageResources = nullptr;
	}
}


FElectraTextureSample::~FElectraTextureSample()
{
	ShutdownPoolable();

	if (Buffer)
	{
		FMemory::Free(Buffer);
	}

	CleanupImageResources();
}

const void* FElectraTextureSample::GetBuffer()
{
	return Buffer;
}


FRHITexture* FElectraTextureSample::GetTexture() const
{
	return Texture.GetReference();
}

IMediaTextureSampleConverter* FElectraTextureSample::GetMediaTextureSampleConverter()
{
	return this;
}

FIntPoint FElectraTextureSample::GetDim() const
{
	return IElectraTextureSampleBase::GetDim();
}


EMediaTextureSampleFormat FElectraTextureSample::GetFormat() const
{
	return GetPixelFormat() == EPixelFormat::PF_A2B10G10R10 ? EMediaTextureSampleFormat::CharBGR10A2 : EMediaTextureSampleFormat::CharBGRA;
}


uint32 FElectraTextureSample::GetStride() const
{
	// note: we expect RGBA8 or RGB10A2 -> it's always 32 bits
	return GetDim().X * sizeof(uint32);
}


void FElectraTextureSample::CopyFromExternalTextureOES(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, FTextureRHIRef& InSrcTexture, const FVector2f& InScale, const FVector2f& InOffset)
{
	FLinearColor Offset = { InOffset.X, InOffset.Y, 0.f, 0.f };
	FLinearColor ScaleRotation = { InScale.X, 0.f,
								  0.f, InScale.Y };

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	FRHITexture* RenderTarget = InDstTexture.GetReference();

	RHICmdList.Transition(FRHITransitionInfo(InDstTexture, ERHIAccess::Unknown, ERHIAccess::RTV));

	FRHIRenderPassInfo RPInfo(RenderTarget, ERenderTargetActions::DontLoad_Store);
	RHICmdList.BeginRenderPass(RPInfo, TEXT("ConvertMedia_ExternalTexture"));
	{
		const FIntPoint OutputDim = GetOutputDim();

		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		RHICmdList.SetViewport(0, 0, 0.0f, (float)OutputDim.X, (float)OutputDim.Y, 1.0f);

		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_RGBA, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI();
		GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

		// configure media shaders
		auto ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FMediaShadersVS> VertexShader(ShaderMap);

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GMediaVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();

		FSamplerStateInitializerRHI InSamplerStateInitializer(SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp);
		FSamplerStateRHIRef InSamplerState = RHICreateSamplerState(InSamplerStateInitializer);

		TShaderMapRef<FReadTextureExternalPS> CopyShader(ShaderMap);
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = CopyShader.GetPixelShader();
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
		SetShaderParametersLegacyPS(RHICmdList, CopyShader, InSrcTexture, InSamplerState, ScaleRotation, Offset);

		// draw full size quad into render target
		FBufferRHIRef VertexBuffer = CreateTempMediaVertexBuffer(RHICmdList);
		RHICmdList.SetStreamSource(0, VertexBuffer, 0);
		// set viewport to RT size
		RHICmdList.SetViewport(0, 0, 0.0f, (float)OutputDim.X, (float)OutputDim.Y, 1.0f);

		RHICmdList.DrawPrimitive(0, 2, 1);
	}
	RHICmdList.EndRenderPass();

	RHICmdList.Transition(FRHITransitionInfo(InDstTexture, ERHIAccess::RTV, ERHIAccess::SRVMask));
}

void  FElectraTextureSample::CopyFromExternalTextureVulkan(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, FTextureRHIRef& InSrcTexture, const FVector2f& InScale, const FVector2f& InOffset)
{
	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	FRHITexture* RenderTarget = InDstTexture;

	if (!Fence)
	{
		Fence = RHICreateGPUFence(TEXT("CopyFromExternalTextureVulkan"));
	}

	RHICmdList.Transition(FRHITransitionInfo(InDstTexture, ERHIAccess::Unknown, ERHIAccess::RTV));

	FRHIRenderPassInfo RPInfo(RenderTarget, ERenderTargetActions::DontLoad_Store);
	RHICmdList.BeginRenderPass(RPInfo, TEXT("ConvertMedia"));
	{
		const FIntPoint OutputDim = GetOutputDim();

		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		RHICmdList.SetViewport(0, 0, 0.0f, (float)OutputDim.X, (float)OutputDim.Y, 1.0f);

		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_RGBA, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI();
		GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

		// configure media shaders
		auto ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FMediaShadersVS> VertexShader(ShaderMap);

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GMediaVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();

		auto YUVMtx = GetSampleToRGBMatrix();
		FMatrix44f ColorSpaceMtx;
		//------------------------------------------------------------------------
		//GetColorSpaceConversionMatrixForSample(this, ColorSpaceMtx);
		{
			const UE::Color::FColorSpace& Working = UE::Color::FColorSpace::GetWorking();

			if (GetMediaTextureSampleColorConverter())
			{
				ColorSpaceMtx = FMatrix44f::Identity;
			}
			else
			{
				ColorSpaceMtx = UE::Color::Transpose<float>(UE::Color::FColorSpaceTransform(GetSourceColorSpace(), Working));
			}

			float NF = GetHDRNitsNormalizationFactor();
			if (NF != 1.0f)
			{
				ColorSpaceMtx = ColorSpaceMtx.ApplyScale(NF);
			}
		}
		//------------------------------------------------------------------------

		TShaderMapRef<FVYUConvertPS> ConvertShader(ShaderMap);
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
		SetShaderParametersLegacyPS(RHICmdList, ConvertShader, InSrcTexture, OutputDim, YUVMtx, GetEncodingType(), ColorSpaceMtx);


		// draw full size quad into render target
		FBufferRHIRef VertexBuffer = CreateTempMediaVertexBuffer(RHICmdList);
		RHICmdList.SetStreamSource(0, VertexBuffer, 0);
		// set viewport to RT size
		RHICmdList.SetViewport(0, 0, 0.0f, (float)OutputDim.X, (float)OutputDim.Y, 1.0f);

		RHICmdList.DrawPrimitive(0, 2, 1);
	}
	RHICmdList.EndRenderPass();

	RHICmdList.Transition(FRHITransitionInfo(RenderTarget, ERHIAccess::RTV, ERHIAccess::SRVGraphics));

	RHICmdList.WriteGPUFence(Fence);
}


uint32 FElectraTextureSample::GetConverterInfoFlags() const
{
	return ConverterInfoFlags_Default;
}

bool FElectraTextureSample::Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints)
{
	if (Support->UseGpuOutputPath())
	{
		bQueuedForConversion = ConvertGpuOutputPath(RHICmdList, InDstTexture, Hints);
		return bQueuedForConversion;
	}
	else
		return ConvertCpuOutputPath(RHICmdList, InDstTexture, Hints);
}

bool FElectraTextureSample::ConvertGpuOutputPath(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints)
{
	check(IsInRenderingThread());

	if (GDynamicRHI->RHIIsRenderingSuspended() || ImageResources == nullptr)
	{
		return false;
	}

	jobject HardwareBufferObj = Support->ImageResources_GetHardwareBuffer(ImageResources);
	AHardwareBuffer* HardwareBuffer = AHardwareBuffer_fromHardwareBuffer(FAndroidApplication::GetJavaEnv(), HardwareBufferObj);
	ensure(HardwareBuffer);
	AHardwareBuffer_acquire(HardwareBuffer);

	//check(VideoDecoderOutputAndroid->GetOutputType() == FVideoDecoderOutputAndroid::EOutputType::DirectToSurfaceAsQueue);
	if (FAndroidMisc::ShouldUseVulkan())
	{
		Texture = InitializeTextureVulkan(HardwareBuffer);
	}
	else
	{
		Texture = InitializeTextureOES(HardwareBuffer);
	}

	ensure(Texture);

	if (FAndroidMisc::ShouldUseVulkan())
	{
		FVector2f Scale, Offset;
		Support->ImageResources_GetScaleOffset(ImageResources, Scale, Offset);

		CopyFromExternalTextureVulkan(RHICmdList, InDstTexture, Texture, Scale, Offset);
	}
	else
	{
		FVector2f Scale, Offset;
		Support->ImageResources_GetScaleOffset(ImageResources, Scale, Offset);

		//FOR NOW(?) THIS IS DONE HERE TO MAKE SURE WE HAVE EASY ACCESS TO THE SCALE/OFFSET/ROTATION VALUES FOR EACH SAMPLE
		//(the code using a map & GUID lookup assumes ONE "current" value per player... which does entirely NOT work in reality (queue of frames))
		CopyFromExternalTextureOES(RHICmdList, InDstTexture, Texture, Scale, Offset);
	}

	AHardwareBuffer_release(HardwareBuffer);
	FAndroidApplication::GetJavaEnv()->DeleteLocalRef(HardwareBufferObj);
	return true;
}


bool FElectraTextureSample::ConvertCpuOutputPath(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints)
{
	if (GDynamicRHI->RHIIsRenderingSuspended())
	{
		return false;
	}

	TRefCountPtr<FRHITexture> InputTexture;

	// Either use a texture we have around as a payload or make a temporary one from buffer contents...
	if (!Texture.IsValid())
	{
		auto SampleDim = GetDim();

		// Make a source texture so we can convert from it...
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("FMediaTextureResource"), SampleDim, GetPixelFormat())
			.SetInitialState(ERHIAccess::SRVMask);
		InputTexture = RHICmdList.CreateTexture(Desc);
		if (!InputTexture.IsValid())
		{
			return false;
		}

		// copy sample data to input render target
		FUpdateTextureRegion2D Region(0, 0, 0, 0, SampleDim.X, SampleDim.Y);
		RHICmdList.UpdateTexture2D(InputTexture, 0, Region, GetStride(), (const uint8*)GetBuffer());
	}
	else
	{
		InputTexture = Texture;
	}

	RHI_BREADCRUMB_EVENT_STAT(RHICmdList, MediaAndroidDecoder_Convert, "AndroidMediaOutputConvertTexture");

	const FIntPoint Dim = GetDim();
	const FIntPoint OutputDim = GetOutputDim();

	RHICmdList.Transition(FRHITransitionInfo(InDstTexture, ERHIAccess::Unknown, ERHIAccess::RTV));
	FRHIRenderPassInfo RPInfo(InDstTexture, ERenderTargetActions::DontLoad_Store);

	FBufferRHIRef TempVB = CreateTempMediaVertexBuffer(RHICmdList);

	RHICmdList.BeginRenderPass(RPInfo, TEXT("AndroidProcessVideo"));

	// Update viewport.
	RHICmdList.SetViewport(0, 0, 0.f, OutputDim.X, OutputDim.Y, 1.f);

	// Setup conversion from Rec2020 to current working color space
	const UE::Color::FColorSpace& Working = UE::Color::FColorSpace::GetWorking();
	FMatrix44f ColorSpaceMtx = UE::Color::Transpose<float>(UE::Color::FColorSpaceTransform(GetSourceColorSpace(), Working));
	if (GetEncodingType() == UE::Color::EEncoding::ST2084)
	{
		// Normalize output (e.g. 80 or 100 nits == 1.0)
		ColorSpaceMtx = ColorSpaceMtx.ApplyScale(GetHDRNitsNormalizationFactor());
	}

	// Get shaders.
	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FRGBConvertPS> PixelShader(GlobalShaderMap);
	TShaderMapRef<FMediaShadersVS> VertexShader(GlobalShaderMap);

	// Set the graphic pipeline state.
	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
	GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;
	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GMediaVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	// Update shader uniform parameters.
	SetShaderParametersLegacyPS(RHICmdList, PixelShader, InputTexture, OutputDim, GetEncodingType(), ColorSpaceMtx);

	RHICmdList.SetStreamSource(0, TempVB, 0);

	RHICmdList.DrawPrimitive(0, 2, 1);

	RHICmdList.EndRenderPass();
	RHICmdList.Transition(FRHITransitionInfo(InDstTexture, ERHIAccess::RTV, ERHIAccess::SRVMask));

	return true;
}

#endif
