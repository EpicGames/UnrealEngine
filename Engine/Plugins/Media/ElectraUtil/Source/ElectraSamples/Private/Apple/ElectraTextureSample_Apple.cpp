// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

#if PLATFORM_MAC || PLATFORM_IOS

#include "ElectraTextureSample.h"

#if WITH_ENGINE
#include "RenderingThread.h"
#include "RHI.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIStaticStates.h"
#include "MediaShaders.h"
#include "Containers/ResourceArray.h"
#include "PipelineStateCache.h"
#else
#include "Containers/Array.h"
#endif

THIRD_PARTY_INCLUDES_START
#include "MetalInclude.h"
THIRD_PARTY_INCLUDES_END

#include "IMetalDynamicRHI.h"

#if WITH_ENGINE
extern void SafeReleaseMetalObject(NS::Object* Object);
#endif

// ------------------------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------------------------

FElectraMediaTexConvApple::FElectraMediaTexConvApple()
{
#if WITH_ENGINE
	MetalTextureCache = nullptr;
#endif
}

FElectraMediaTexConvApple::~FElectraMediaTexConvApple()
{
#if WITH_ENGINE
	if (MetalTextureCache)
	{
		CVMetalTextureCacheRef TextureCacheCopy = MetalTextureCache;
		SafeReleaseMetalObject((__bridge NS::Object*)TextureCacheCopy);
	}
#endif
}



#if WITH_ENGINE
void FElectraMediaTexConvApple::ConvertTexture(FTextureRHIRef& InDstTexture, CVImageBufferRef InImageBufferRef, bool bFullRange, EMediaTextureSampleFormat Format, const FMatrix44f& YUVMtx, const UE::Color::FColorSpace& SourceColorSpace, UE::Color::EEncoding EncodingType, float NormalizationFactor)
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	const int32 FrameHeight = CVPixelBufferGetHeight(InImageBufferRef);
	const int32 FrameWidth = CVPixelBufferGetWidth(InImageBufferRef);
	const int32 FrameStride = CVPixelBufferGetBytesPerRow(InImageBufferRef);

	// We have to support Metal for this object now
	check(IsMetalPlatform(GMaxRHIShaderPlatform));
	{
		if (!MetalTextureCache)
		{
            id<MTLDevice> Device = (__bridge id<MTLDevice>)GetIMetalDynamicRHI()->RHIGetDevice();
			check(Device);

			CVReturn Return = CVMetalTextureCacheCreate(kCFAllocatorDefault, nullptr, Device, nullptr, &MetalTextureCache);
			check(Return == kCVReturnSuccess);
		}
		check(MetalTextureCache);

		if (CVPixelBufferIsPlanar(InImageBufferRef))
		{
			// Planar data: YbCrCb 420 etc. (NV12 / P010)

			bool bIs8Bit = (Format == EMediaTextureSampleFormat::CharNV12);

			// Expecting BiPlanar kCVPixelFormatType_420YpCbCr8BiPlanar Full/Video
			check(CVPixelBufferGetPlaneCount(InImageBufferRef) == 2);

			int32 YWidth = CVPixelBufferGetWidthOfPlane(InImageBufferRef, 0);
			int32 YHeight = CVPixelBufferGetHeightOfPlane(InImageBufferRef, 0);

			CVMetalTextureRef YTextureRef = nullptr;
			CVReturn Result = CVMetalTextureCacheCreateTextureFromImage(kCFAllocatorDefault, MetalTextureCache, InImageBufferRef, nullptr, bIs8Bit ? MTLPixelFormatR8Unorm : MTLPixelFormatR16Unorm, YWidth, YHeight, 0, &YTextureRef);
			check(Result == kCVReturnSuccess);
			check(YTextureRef);

			int32 UVWidth = CVPixelBufferGetWidthOfPlane(InImageBufferRef, 1);
			int32 UVHeight = CVPixelBufferGetHeightOfPlane(InImageBufferRef, 1);

			CVMetalTextureRef UVTextureRef = nullptr;
			Result = CVMetalTextureCacheCreateTextureFromImage(kCFAllocatorDefault, MetalTextureCache, InImageBufferRef, nullptr, bIs8Bit ? MTLPixelFormatRG8Unorm : MTLPixelFormatRG16Unorm, UVWidth, UVHeight, 1, &UVTextureRef);
			check(Result == kCVReturnSuccess);
			check(UVTextureRef);

			// Metal can upload directly from an IOSurface to a 2D texture
			IMetalDynamicRHI* MetalDynamicRHI = GetIMetalDynamicRHI();

			TRefCountPtr<FRHITexture> YTex = MetalDynamicRHI->RHICreateTexture2DFromCVMetalTexture(bIs8Bit ? PF_G8 : PF_G16, ETextureCreateFlags::NoTiling | ETextureCreateFlags::ShaderResource, FClearValueBinding::White, YTextureRef);
			TRefCountPtr<FRHITexture> UVTex = MetalDynamicRHI->RHICreateTexture2DFromCVMetalTexture(bIs8Bit ? PF_R8G8 : PF_G16R16, ETextureCreateFlags::NoTiling | ETextureCreateFlags::ShaderResource, FClearValueBinding::White, UVTextureRef);

			{
				// configure media shaders
				auto GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

				RHICmdList.Transition(FRHITransitionInfo(InDstTexture, ERHIAccess::SRVMask, ERHIAccess::RTV));
				FRHIRenderPassInfo RPInfo(InDstTexture, ERenderTargetActions::DontLoad_Store);
				RHICmdList.BeginRenderPass(RPInfo, TEXT("AvfMediaSampler"));
				{
					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

					GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_RGBA, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI();
					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GMediaVertexDeclaration.VertexDeclarationRHI;
					GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

					// Setup conversion from source color space (i.e. Rec2020) to current working color space
					const UE::Color::FColorSpace& Working = UE::Color::FColorSpace::GetWorking();
					FMatrix44f ColorSpaceMtx = UE::Color::Transpose<float>(UE::Color::FColorSpaceTransform(SourceColorSpace, Working));
					ColorSpaceMtx = ColorSpaceMtx.ApplyScale(NormalizationFactor);

					if (Format == EMediaTextureSampleFormat::CharNV12)
					{
						//
						// NV12
						//

						TShaderMapRef<FMediaShadersVS> VertexShader(GlobalShaderMap);
						TShaderMapRef<FNV12ConvertPS> PixelShader(GlobalShaderMap);
						GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
						GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
						SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

						FShaderResourceViewRHIRef Y_SRV = RHICmdList.CreateShaderResourceView(
							YTex,
							FRHIViewDesc::CreateTextureSRV()
								.SetDimensionFromTexture(YTex)
								.SetMipRange(0, 1)
								.SetFormat(PF_G8));
						FShaderResourceViewRHIRef UV_SRV = RHICmdList.CreateShaderResourceView(
							UVTex,
							FRHIViewDesc::CreateTextureSRV()
								.SetDimensionFromTexture(UVTex)
								.SetMipRange(0, 1)
								.SetFormat(PF_R8G8));

						SetShaderParametersLegacyPS(RHICmdList, PixelShader, FIntPoint(YWidth, YHeight), Y_SRV, UV_SRV, FIntPoint(YWidth, YHeight), YUVMtx, EncodingType, ColorSpaceMtx, false);
					}
					else
					{
						//
						// P010
						//

						TShaderMapRef<FP010ConvertPS> PixelShader(GlobalShaderMap);
						TShaderMapRef<FMediaShadersVS> VertexShader(GlobalShaderMap);
						GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
						GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
						SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

						FShaderResourceViewRHIRef Y_SRV = RHICmdList.CreateShaderResourceView(
							YTex,
							FRHIViewDesc::CreateTextureSRV()
								.SetDimensionFromTexture(YTex)
								.SetMipRange(0, 1)
								.SetFormat(PF_G16));
						FShaderResourceViewRHIRef UV_SRV = RHICmdList.CreateShaderResourceView(
							UVTex,
							FRHIViewDesc::CreateTextureSRV()
								.SetDimensionFromTexture(UVTex)
								.SetMipRange(0, 1)
								.SetFormat(PF_G16R16));

						// Update shader uniform parameters.
						SetShaderParametersLegacyPS(RHICmdList, PixelShader, FIntPoint(YWidth, YHeight), Y_SRV, UV_SRV, FIntPoint(YWidth, YHeight), YUVMtx, ColorSpaceMtx, EncodingType);
					}


					FBufferRHIRef VertexBuffer = CreateTempMediaVertexBuffer(RHICmdList);
					RHICmdList.SetStreamSource(0, VertexBuffer, 0);
					RHICmdList.SetViewport(0, 0, 0.0f, YWidth, YHeight, 1.0f);

					RHICmdList.DrawPrimitive(0, 2, 1);
				}
				RHICmdList.EndRenderPass();
				RHICmdList.Transition(FRHITransitionInfo(InDstTexture, ERHIAccess::RTV, ERHIAccess::SRVMask));
			}
			CFRelease(YTextureRef);
			CFRelease(UVTextureRef);
		}
		else
		{
			if (Format == EMediaTextureSampleFormat::Y416)
			{
				//
				// YCbCrA, 16-bit (4:4:4:4)
				//

				//
				// Grab data directly from image buffer reference
				// (this will create the output texture here - but will always use the settings here, as data is not converted)
				//
				int32 Width = CVPixelBufferGetWidth(InImageBufferRef);
				int32 Height = CVPixelBufferGetHeight(InImageBufferRef);

				CVMetalTextureRef TextureRef = nullptr;
				CVReturn Result = CVMetalTextureCacheCreateTextureFromImage(kCFAllocatorDefault, MetalTextureCache, InImageBufferRef, nullptr, MTLPixelFormatRGBA16Unorm, Width, Height, 0, &TextureRef);
				check(Result == kCVReturnSuccess);
				check(TextureRef);

				IMetalDynamicRHI* MetalDynamicRHI = GetIMetalDynamicRHI();

				InDstTexture = MetalDynamicRHI->RHICreateTexture2DFromCVMetalTexture(PF_R16G16B16A16_UNORM, ETextureCreateFlags::NoTiling | ETextureCreateFlags::ShaderResource, FClearValueBinding::White, TextureRef);
				CFRelease(TextureRef);
			}
			else
			{
				//
				// sRGB
				//

				//
				// Grab data directly from image buffer reference
				// (this will create the output texture here - but will always use the settings here, as data is not converted)
				//
				int32 Width = CVPixelBufferGetWidth(InImageBufferRef);
				int32 Height = CVPixelBufferGetHeight(InImageBufferRef);

				CVMetalTextureRef TextureRef = nullptr;
				CVReturn Result = CVMetalTextureCacheCreateTextureFromImage(kCFAllocatorDefault, MetalTextureCache, InImageBufferRef, nullptr, MTLPixelFormatBGRA8Unorm_sRGB, Width, Height, 0, &TextureRef);
				check(Result == kCVReturnSuccess);
				check(TextureRef);

				IMetalDynamicRHI* MetalDynamicRHI = GetIMetalDynamicRHI();

				InDstTexture = MetalDynamicRHI->RHICreateTexture2DFromCVMetalTexture(PF_B8G8R8A8, ETextureCreateFlags::NoTiling | ETextureCreateFlags::ShaderResource, FClearValueBinding::White, TextureRef);
				CFRelease(TextureRef);
			}
		}
	}
}
#endif

// ------------------------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------------------------

FElectraTextureSample::FElectraTextureSample(const TWeakPtr<FElectraMediaTexConvApple, ESPMode::ThreadSafe>& InTexConv)
	: TexConv(InTexConv)
{
}

FElectraTextureSample::~FElectraTextureSample()
{
	ShutdownPoolable();
}


FRHITexture* FElectraTextureSample::GetTexture() const
{
	return nullptr;
}

void FElectraTextureSample::ShutdownPoolable()
{
	if (bWasShutDown)
	{
		return;
	}
	Buffer.Reset();
	if (ImageBufferRef)
	{
		CFRelease(ImageBufferRef);
		ImageBufferRef = nullptr;
	}
	IElectraTextureSampleBase::ShutdownPoolable();
}

bool FElectraTextureSample::FinishInitialization()
{
	if (ImageBufferRef)
    {
        OSType PixelFormat = CVPixelBufferGetPixelFormatType(ImageBufferRef);
		switch(PixelFormat)
		{
			case kCVPixelFormatType_4444AYpCbCr16:
			{
				SampleFormat = EMediaTextureSampleFormat::Y416;
				break;
			}
			case kCVPixelFormatType_420YpCbCr8BiPlanarVideoRange:
			case kCVPixelFormatType_420YpCbCr8BiPlanarFullRange:
			{
				SampleFormat = EMediaTextureSampleFormat::CharNV12;
				break;
			}
			default:
			{
				SampleFormat = EMediaTextureSampleFormat::P010;
				break;
			}
		}
		return true;
    }
	else
	{
		switch(PixelFormat)
		{
			case EPixelFormat::PF_DXT1:
			{
				SampleFormat = EMediaTextureSampleFormat::DXT1;
				break;
			}
			case EPixelFormat::PF_DXT5:
			{
				switch(PixelFormatEncoding)
				{
					case EElectraTextureSamplePixelEncoding::YCoCg:
					{
						SampleFormat = EMediaTextureSampleFormat::YCoCg_DXT5;
						break;
					}
					case EElectraTextureSamplePixelEncoding::YCoCg_Alpha:
					{
						SampleFormat = EMediaTextureSampleFormat::YCoCg_DXT5_Alpha_BC4;
						break;
					}
					case EElectraTextureSamplePixelEncoding::Native:
					{
						SampleFormat = EMediaTextureSampleFormat::DXT5;
						break;
					}
					default:
					{
						SampleFormat = EMediaTextureSampleFormat::Undefined;
						return false;
					}
				}
				break;
			}
			case EPixelFormat::PF_BC4:
			{
				SampleFormat = EMediaTextureSampleFormat::BC4;
				break;
			}
			case EPixelFormat::PF_NV12:
			{
				SampleFormat = EMediaTextureSampleFormat::CharNV12;
				break;
			}
			default:
			{
				SampleFormat = EMediaTextureSampleFormat::Undefined;
				return false;
			}
		}
		return true;
	}
}


EMediaTextureSampleFormat FElectraTextureSample::GetFormat() const
{
	return SampleFormat;
}

const void* FElectraTextureSample::GetBuffer()
{
	return Buffer ? Buffer->GetData() : nullptr;
}

uint32 FElectraTextureSample::GetStride() const
{
	return Stride;
}

IMediaTextureSampleConverter* FElectraTextureSample::GetMediaTextureSampleConverter()
{
	return ImageBufferRef != nullptr ? this : nullptr;
}

uint32 FElectraTextureSample::GetConverterInfoFlags() const
{
	return ImageBufferRef && CVPixelBufferIsPlanar(ImageBufferRef) ? ConverterInfoFlags_Default : ConverterInfoFlags_WillCreateOutputTexture;
}

bool FElectraTextureSample::Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints)
{
	if (TSharedPtr<FElectraMediaTexConvApple, ESPMode::ThreadSafe> PinnedTexConv = TexConv.Pin())
	{
		PinnedTexConv->ConvertTexture(InDstTexture, ImageBufferRef,
			Colorimetry.IsValid() ? bIsFullRange : true,
			GetFormat(), GetSampleToRGBMatrix(), GetSourceColorSpace(), GetEncodingType(), GetHDRNitsNormalizationFactor());
		return true;
	}
	return false;
}

#endif
