// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraAudioSample.h"

FElectraAudioSample::~FElectraAudioSample()
{
	FMemory::Free(Buffer);
}

bool FElectraAudioSample::AllocateFor(EMediaAudioSampleFormat InFormat, uint32 InNumChannels, uint32 InNumFrames)
{
	check(InFormat == EMediaAudioSampleFormat::Float);

	uint32 NumBytesNeeded = sizeof(float) * InNumChannels * InNumFrames;
	if (NumBytesNeeded > NumBytesAllocated)
	{
		Buffer = FMemory::Realloc(Buffer, NumBytesNeeded);
		if (!Buffer)
		{
			NumBytesAllocated = 0;
			return false;
		}
		NumBytesAllocated = NumBytesNeeded;
	}
	MediaAudioSampleFormat = InFormat;
	NumChannels = InNumChannels;
	NumFrames = InNumFrames;
	return true;
}

void FElectraAudioSample::SetParameters(uint32 InSampleRate, const FMediaTimeStamp& InTime, const FTimespan& InDuration)
{
	SampleRate = InSampleRate;
	MediaTimeStamp = InTime;
	Duration = InDuration;
}

#if !UE_SERVER
void FElectraAudioSample::ShutdownPoolable()
{
	ReleaseDelegate.ExecuteIfBound(this);
	ReleaseDelegate.Unbind();
}
#endif
