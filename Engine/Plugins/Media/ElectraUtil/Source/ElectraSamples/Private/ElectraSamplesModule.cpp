// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraSamplesModule.h"

#include "CoreTypes.h"
#include "Modules/ModuleManager.h"
#include "IElectraSamplesModule.h"

#define LOCTEXT_NAMESPACE "ElectraBaseModule"

DEFINE_LOG_CATEGORY(LogElectraSamples);

// -----------------------------------------------------------------------------------------------------------------------------------

class FElectraSamplesModule: public IElectraSamplesModule
{
public:
	// IModuleInterface interface
	void StartupModule() override
	{
	}

	void ShutdownModule() override
	{
	}
};

IMPLEMENT_MODULE(FElectraSamplesModule, ElectraSamples);

#undef LOCTEXT_NAMESPACE
