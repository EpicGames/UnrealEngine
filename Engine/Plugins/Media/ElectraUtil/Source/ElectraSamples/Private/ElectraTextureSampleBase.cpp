// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/Build.h"

#include "ColorManagement/TransferFunctions.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Platform.h"
#include "ElectraTextureSample.h"
#include "ElectraTextureSampleUtils.h"
#include "ElectraSamplesModule.h"

#if !UE_SERVER

#include "MediaDecoderOutput.h"
#include "MediaVideoDecoderOutput.h"

// -------------------------------------------------------------------------------------------------------------------------------------------------------

namespace
{

static TOptional<FTimecode> CreateTimecodeFromMPEGDefinition(TOptional<FFrameRate>& OutFramerate, const IVideoDecoderTimecode::FMPEGDefinition* InMPEGTimecode)
{
	if (InMPEGTimecode->timing_info_present_flag)
	{
		// Compute TotalSeconds directly as a double division to avoid precision loss from
		// FTimeValue's intermediate conversion to hundred-nanosecond integer ticks (HNS).
		// FTimeValue::SetFromND() truncates via integer division (clockTimestamp * 10000000 / time_scale),
		// which for time_scale values that don't evenly divide 10000000 (e.g. 60) produces a systematic
		// downward bias that manifests as near-1.0 subframe values on the resulting FTimecode.
		const double TotalSeconds = static_cast<double>(InMPEGTimecode->clockTimestamp) / static_cast<double>(InMPEGTimecode->time_scale);
		OutFramerate = FFrameRate(InMPEGTimecode->time_scale, InMPEGTimecode->num_units_in_tick);
		const FFrameTime FrameTime = OutFramerate.GetValue().AsFrameTime(TotalSeconds);
		const bool bDropFrame = InMPEGTimecode->ct_type > 1;
		return FTimecode::FromFrameTime(FrameTime, OutFramerate.GetValue(), bDropFrame);
	}
	return TOptional<FTimecode>();
}

}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
static TAutoConsoleVariableDeprecated_NoReplace<float> CVarElectraHdrWhiteLevel(
	TEXT("Electra.HDR.WhiteLevel"),
	MediaTextureSample::kLinearToNitsScale_BT2408,
	TEXT("5.8"),
	EShadowCVarBehavior::Ensure,
	TEXT("Electra.HDR.WhiteLevel has no effect: Electra HDR media normalizes BT.2408 Reference White (203 nits) to scene-linear 1.0. Any additional white-level adjustment should be applied in the consuming material."));
PRAGMA_ENABLE_DEPRECATION_WARNINGS

static TAutoConsoleVariable<int32> CVarElectraHdrToneMapMethod(
	TEXT("Electra.HDR.ToneMapMethod"),
	static_cast<int32>(MediaShaders::EToneMapMethod::Hable),
	TEXT("Tone mapping method applied on source HDR media:\n")
	TEXT(" 0: None\n")
	TEXT(" 1: Hable (default)\n")
	TEXT(" 2: SimpleReinhard\n"),
	ECVF_Default);

// -------------------------------------------------------------------------------------------------------------------------------------------------------

void IElectraTextureSampleBase::InitializeCommon(const Electra::FParamDict& InParams)
{
	// HDR info
	if (InParams.HaveKey(IDecoderOutputOptionNames::HDRInfo))
	{
		HDRInfo = InParams.GetValue(IDecoderOutputOptionNames::HDRInfo).GetSharedPointer<const IVideoDecoderHDRInformation>();
	}
	// Colorimetry info
	if (InParams.HaveKey(IDecoderOutputOptionNames::Colorimetry))
	{
		Colorimetry = InParams.GetValue(IDecoderOutputOptionNames::Colorimetry).GetSharedPointer<const IVideoDecoderColorimetry>();
	}
	// Timecode
	if (InParams.HaveKey(IDecoderOutputOptionNames::Timecode))
	{
		Timecode = CreateTimecodeFromMPEGDefinition(Framerate, InParams.GetValue(IDecoderOutputOptionNames::Timecode).GetSharedPointer<const IVideoDecoderTimecode>()->GetMPEGDefinition());
	}
	else if (InParams.HaveKey(IDecoderOutputOptionNames::TMCDTimecode))
	{
		Timecode = InParams.GetValue(IDecoderOutputOptionNames::TMCDTimecode).GetTimecode();
		Framerate = InParams.GetValue(IDecoderOutputOptionNames::TMCDFramerate).SafeGetFramerate();
	}

	// Get various basic MPEG-style colorimetry values (we default to video range Rec709 SDR)
	uint8 ColorPrimaries = ElectraColorimetryUtils::DefaultMPEGColorPrimaries;
	uint8 TransferCharacteristics = ElectraColorimetryUtils::DefaultMPEGMatrixCoefficients;
	uint8 MatrixCoefficients = ElectraColorimetryUtils::DefaultMPEGTransferCharacteristics;
	if (Colorimetry.IsValid())
	{
		bIsFullRange = Colorimetry->GetMPEGDefinition()->VideoFullRangeFlag != 0;
		ColorPrimaries = Colorimetry->GetMPEGDefinition()->ColourPrimaries;
		TransferCharacteristics = Colorimetry->GetMPEGDefinition()->TransferCharacteristics;
		MatrixCoefficients = Colorimetry->GetMPEGDefinition()->MatrixCoefficients;
	}

	PixelFormat = static_cast<EPixelFormat>(InParams.GetValue(IDecoderOutputOptionNames::PixelFormat).SafeGetInt64((int64)EPixelFormat::PF_Unknown));
	uint8 NumBits = 8;
	if (!IsDXTCBlockCompressedTextureFormat(PixelFormat))
	{
		if (PixelFormat == PF_NV12)
		{
			NumBits = 8;
		}
		else if (PixelFormat == PF_A2B10G10R10)
		{
			NumBits = 10;
		}
		else if (PixelFormat == PF_P010)
		{
			NumBits = 16;
		}
		else
		{
			NumBits = (8 * GPixelFormats[PixelFormat].BlockBytes) / GPixelFormats[PixelFormat].NumComponents;
		}
	}
	PixelFormatEncoding = static_cast<EElectraTextureSamplePixelEncoding>(InParams.GetValue(IDecoderOutputOptionNames::PixelEncoding).SafeGetInt64((int64)EElectraTextureSamplePixelEncoding::Native));
	PixelDataScale = static_cast<float>(InParams.GetValue(IDecoderOutputOptionNames::PixelDataScale).SafeGetDouble(1.0));

	FVector Off = FVector::Zero();
	const FMatrix* Mtx = nullptr;

	// Defaults in case no HDR info is present
	DisplayMasteringColorSpace.Reset();
	DisplayMasteringLuminanceMin = -1.0f;
	DisplayMasteringLuminanceMax = -1.0f;
	MaxCLL = 0;
	MaxFALL = 0;

	// Do we have specific HDR information, so we can assume a standard?
	if (HDRInfo.IsValid())
	{
		// Mastering display info...
		if (auto ColorVolume = HDRInfo->GetMasteringDisplayColourVolume())
		{
			// A few sanity checks on the primaries coordinates (by no means exhaustive, but it should catch a fair share of oddities)
			if (ColorVolume->display_primaries_x[0] > FMath::Max(ColorVolume->display_primaries_x[1], ColorVolume->display_primaries_x[2]) &&	// Red has largest X
				ColorVolume->display_primaries_y[1] > FMath::Max(ColorVolume->display_primaries_y[0], ColorVolume->display_primaries_y[2]) &&	// Green has largest Y
				ColorVolume->display_primaries_x[2] <= ColorVolume->display_primaries_x[0] &&													// Blue's X is smaller than Red's
				ColorVolume->display_primaries_y[2] <= ColorVolume->display_primaries_y[0] &&													// Blue's Y is smaller than Red's
				ColorVolume->display_primaries_x[2] <= ColorVolume->display_primaries_x[1] && 													// Blue's X is smaller or same than Green's
				ColorVolume->display_primaries_x[1] <= ColorVolume->display_primaries_x[0]) 													// Red's X is greater or same than Green's
			{
				DisplayMasteringColorSpace = UE::Color::FColorSpace(FVector2d(ColorVolume->display_primaries_x[0], ColorVolume->display_primaries_y[0]),
														   FVector2d(ColorVolume->display_primaries_x[1], ColorVolume->display_primaries_y[1]),
														   FVector2d(ColorVolume->display_primaries_x[2], ColorVolume->display_primaries_y[2]),
														   FVector2d(ColorVolume->white_point_x, ColorVolume->white_point_y));
			}

			DisplayMasteringLuminanceMin = ColorVolume->min_display_mastering_luminance;
			DisplayMasteringLuminanceMax = ColorVolume->max_display_mastering_luminance;
		}

		// Content light level info...
		if (auto ContentLightLevelInfo = HDRInfo->GetContentLightLevelInfo())
		{
			MaxCLL = ContentLightLevelInfo->max_content_light_level;
			MaxFALL =  ContentLightLevelInfo->max_pic_average_light_level;
		}
	}

	// The sample source color space is always defined by the color primaries value
	SourceColorSpace = UE::Color::FColorSpace(ElectraColorimetryUtils::TranslateMPEGColorPrimaries(ColorPrimaries));

	// Select the YUV-RGB conversion matrix to use
	switch(ElectraColorimetryUtils::TranslateMPEGMatrixCoefficients(MatrixCoefficients))
	{
		case UE::Color::EColorSpace::None:	// ID (RGB)
		{
			// no conversion, data is RGB
			break;
		}
		case UE::Color::EColorSpace::sRGB:
		{
			Mtx = bIsFullRange ? &MediaShaders::YuvToRgbRec709Unscaled : &MediaShaders::YuvToRgbRec709Scaled;
			break;
		}
		case UE::Color::EColorSpace::Rec2020:
		{
			Mtx = bIsFullRange ? &MediaShaders::YuvToRgbRec2020Unscaled : &MediaShaders::YuvToRgbRec2020Scaled;
			break;
		}
		default:
		{
			check(!"*** Unexpected matrix coefficients!");
			Mtx = bIsFullRange ? &MediaShaders::YuvToRgbRec709Unscaled : &MediaShaders::YuvToRgbRec709Scaled;
			break;
		}
	}

	// Get color encoding (sRGB, linear, PQ, HLG...)
	ColorEncoding = ElectraColorimetryUtils::TranslateMPEGTransferCharacteristics(TransferCharacteristics);

	if (Mtx)
	{
		// Select the offsets prior to YUV conversion needed per the incoming data
		switch(NumBits)
		{
			case 8:
			{
				Off = bIsFullRange ? MediaShaders::YUVOffsetNoScale8bits : MediaShaders::YUVOffset8bits;
				break;
			}
			case 10:
			{
				Off = bIsFullRange ? MediaShaders::YUVOffsetNoScale10bits : MediaShaders::YUVOffset10bits;
				break;
			}
			case 16:
			{
				Off = bIsFullRange ? MediaShaders::YUVOffsetNoScale16bits : MediaShaders::YUVOffset16bits;
				break;
			}
			case 32:
			{
				Off = bIsFullRange ? MediaShaders::YUVOffsetNoScaleFloat : MediaShaders::YUVOffsetFloat;
				break;
			}
			default:
			{
				check(!"Unexpected number of bits per channel!");
				break;
			}
		}
	}

	// Correctional scale for input data
	// (data should be placed in the upper 10-bits of the 16-bit texture channels, but some platforms do not do this - they provide a correctional factor here)
	float DataScale = GetSampleDataScale(NumBits == 10);

	// Compute scale to make correct towards the max value (P010 will max out at 0xffc0 not 0xffff - so if it is present we need to adjust the scale a bit)
	float NormScale = PixelFormat == PF_P010 ? 65535.0f / 65472.0f : 1.0f;

	// Matrix to transform sample data to standard YUV values
	FMatrix PreMtx = FMatrix::Identity;
	PreMtx.M[0][0] = DataScale * NormScale;
	PreMtx.M[1][1] = DataScale * NormScale;
	PreMtx.M[2][2] = DataScale * NormScale;
	PreMtx.M[0][3] = -Off.X;
	PreMtx.M[1][3] = -Off.Y;
	PreMtx.M[2][3] = -Off.Z;

	// Combine this with the actual YUV-RGB conversion
	SampleToRgbMtx = FMatrix44f(Mtx ? (*Mtx * PreMtx) : PreMtx);

	// Also store the plain YUV->RGB matrix (pointer) for later reference
	YuvToRgbMtx = Mtx;


	// Get the common values.
	Electra::FTimeValue PTS(InParams.GetValue(IDecoderOutputOptionNames::PTS).GetTimeValue());
	PresentationTimeStamp = FMediaTimeStamp(PTS.GetAsTimespan(), PTS.GetSequenceIndex());

	Duration = InParams.GetValue(IDecoderOutputOptionNames::Duration).GetTimeValue().GetAsTimespan();

	ImageOutputDim.X = (int32)InParams.GetValue(IDecoderOutputOptionNames::Width).SafeGetInt64(0);
	ImageOutputDim.Y = (int32)InParams.GetValue(IDecoderOutputOptionNames::Height).SafeGetInt64(0);
	AspectRatio = ((double)ImageOutputDim.X / (double)ImageOutputDim.Y) * InParams.GetValue(IDecoderOutputOptionNames::AspectRatio).SafeGetDouble(1.0);
	Orientation = (EMediaOrientation)InParams.GetValue(IDecoderOutputOptionNames::Orientation).SafeGetInt64((int64)EMediaOrientation::Original);

	CropLeft = (int32)InParams.GetValue(IDecoderOutputOptionNames::CropLeft).SafeGetInt64(0);
	CropTop = (int32)InParams.GetValue(IDecoderOutputOptionNames::CropTop).SafeGetInt64(0);
	CropRight = (int32)InParams.GetValue(IDecoderOutputOptionNames::CropRight).SafeGetInt64(0);
	CropBottom = (int32)InParams.GetValue(IDecoderOutputOptionNames::CropBottom).SafeGetInt64(0);
	ImageDim.X = ImageOutputDim.X + (CropLeft + CropRight);
	ImageDim.Y = ImageOutputDim.Y + (CropTop + CropBottom);
	// Newly initialized now.
	bWasShutDown = false;
}

bool IElectraTextureSampleBase::FinishInitialization()
{
	return true;
}

IElectraTextureSampleBase::FReleaseDelegate& IElectraTextureSampleBase::GetReleaseDelegate()
{
	return ReleaseDelegate;
}


// -------------------------------------------------------------------------------------------------------------------------------------------------------

IElectraTextureSampleBase::~IElectraTextureSampleBase()
{
	ShutdownPoolable();
}

bool IElectraTextureSampleBase::IsCacheable() const
{
	return true;
}

void IElectraTextureSampleBase::InitializePoolable()
{
}

void IElectraTextureSampleBase::ShutdownPoolable()
{
	if (!bWasShutDown)
	{
		ReleaseDelegate.ExecuteIfBound(this);
		// Clean out what is needed for this element to be reused.
		ReleaseDelegate.Unbind();
		bWasShutDown = true;
	}
}

bool IElectraTextureSampleBase::IsReadyForReuse()
{
	return true;
}

FIntPoint IElectraTextureSampleBase::GetDim() const
{
	return ImageDim;
}

void IElectraTextureSampleBase::SetDim(const FIntPoint& InDim)
{
	ImageDim = InDim;
}

FIntPoint IElectraTextureSampleBase::GetOutputDim() const
{
	return ImageOutputDim;
}

FMediaTimeStamp IElectraTextureSampleBase::GetTime() const
{
	return PresentationTimeStamp;
}

void IElectraTextureSampleBase::SetTime(const FMediaTimeStamp& InTime)
{
	PresentationTimeStamp = InTime;
}

void IElectraTextureSampleBase::SetDuration(const FTimespan& InDuration)
{
	Duration = InDuration;
}

TSharedPtr<const IVideoDecoderColorimetry, ESPMode::ThreadSafe> IElectraTextureSampleBase::GetColorimetry() const
{
	return Colorimetry;
}

EPixelFormat IElectraTextureSampleBase::GetPixelFormat() const
{
	return PixelFormat;
}

EElectraTextureSamplePixelEncoding IElectraTextureSampleBase::GetPixelFormatEncoding() const
{
	return PixelFormatEncoding;
}

FTimespan IElectraTextureSampleBase::GetDuration() const
{
	return Duration;
}

TOptional<FTimecode> IElectraTextureSampleBase::GetTimecode() const
{
	return Timecode;
}

TOptional<FFrameRate> IElectraTextureSampleBase::GetFramerate() const
{
	return Framerate;
}

double IElectraTextureSampleBase::GetAspectRatio() const
{
	return AspectRatio;
}

EMediaOrientation IElectraTextureSampleBase::GetOrientation() const
{
	return Orientation;
}

bool IElectraTextureSampleBase::IsOutputSrgb() const
{
	return ColorEncoding == UE::Color::EEncoding::sRGB;
}

const FMatrix& IElectraTextureSampleBase::GetYUVToRGBMatrix() const
{
	return YuvToRgbMtx ? *YuvToRgbMtx : FMatrix::Identity;
}

bool IElectraTextureSampleBase::GetFullRange() const
{
	return bIsFullRange;
}

FMatrix44f IElectraTextureSampleBase::GetSampleToRGBMatrix() const
{
	return SampleToRgbMtx;
}

const UE::Color::FColorSpace& IElectraTextureSampleBase::GetSourceColorSpace() const
{
	return SourceColorSpace;
}

UE::Color::EEncoding IElectraTextureSampleBase::GetEncodingType() const
{
	return ColorEncoding;
}

float IElectraTextureSampleBase::GetHDRNitsNormalizationFactor() const
{
	// Electra HDR samples follow BT.2408 per MPEG transfer-characteristics conventions.
	// Non-HDR encodings resolve to 1.0 (pass-through) via the helper.
	return UE::Color::GetReferenceWhiteLinearScale(GetEncodingType(), UE::Color::EReferenceWhite::BT2408);
}

bool IElectraTextureSampleBase::GetDisplayMasteringLuminance(float& OutMin, float& OutMax) const
{
	if (DisplayMasteringLuminanceMin < 0.0f && DisplayMasteringLuminanceMax < 0.0f)
	{
		return false;
	}
	OutMin = DisplayMasteringLuminanceMin;
	OutMax = DisplayMasteringLuminanceMax;
	return true;
}

TOptional<UE::Color::FColorSpace> IElectraTextureSampleBase::GetDisplayMasteringColorSpace() const
{
	return DisplayMasteringColorSpace;
}

bool IElectraTextureSampleBase::GetMaxLuminanceLevels(uint16& OutCLL, uint16& OutFALL) const
{
	if (MaxCLL == 0 && MaxFALL == 0)
	{
		return false;
	}
	OutCLL = MaxCLL;
	OutFALL = MaxFALL;
	return true;
}

MediaShaders::EToneMapMethod IElectraTextureSampleBase::GetToneMapMethod() const
{
	if (GetEncodingType() == UE::Color::EEncoding::sRGB || GetEncodingType() == UE::Color::EEncoding::Linear)
	{
		return MediaShaders::EToneMapMethod::None;
	}
	else
	{
		const int32 ToneMapMethod = FMath::Clamp(CVarElectraHdrToneMapMethod->GetInt(), 0, static_cast<int32>(MediaShaders::EToneMapMethod::MAX) - 1);

		return static_cast<MediaShaders::EToneMapMethod>(ToneMapMethod);
	}
}

float IElectraTextureSampleBase::GetSampleDataScale(bool b10Bit) const
{
	return PixelDataScale;
}


#else

IElectraTextureSampleBase::~IElectraTextureSampleBase()
{ }

void IElectraTextureSampleBase::InitializeCommon(const Electra::FParamDict& InParams)
{ }

bool IElectraTextureSampleBase::FinishInitialization()
{ return true; }

IElectraTextureSampleBase::FReleaseDelegate& IElectraTextureSampleBase::GetReleaseDelegate()
{ return ReleaseDelegate; }

bool IElectraTextureSampleBase::IsCacheable() const
{ return true; }

FIntPoint IElectraTextureSampleBase::GetDim() const
{ return FIntPoint(); }

void IElectraTextureSampleBase::SetDim(const FIntPoint& InDim)
{ }

FIntPoint IElectraTextureSampleBase::GetOutputDim() const
{ return FIntPoint(); }

FMediaTimeStamp IElectraTextureSampleBase::GetTime() const
{ return FMediaTimeStamp(); }

void IElectraTextureSampleBase::SetTime(const FMediaTimeStamp& InTime)
{ }

void IElectraTextureSampleBase::SetDuration(const FTimespan& InDuration)
{ }

TSharedPtr<const IVideoDecoderColorimetry, ESPMode::ThreadSafe> IElectraTextureSampleBase::GetColorimetry() const
{ return nullptr; }

EPixelFormat IElectraTextureSampleBase::GetPixelFormat() const
{ return EPixelFormat::PF_Unknown; }

EElectraTextureSamplePixelEncoding IElectraTextureSampleBase::GetPixelFormatEncoding() const
{ return EElectraTextureSamplePixelEncoding::Native; }

FTimespan IElectraTextureSampleBase::GetDuration() const
{ return FTimespan(); }

TOptional<FTimecode> IElectraTextureSampleBase::GetTimecode() const
{ return TOptional<FTimecode>(); }

TOptional<FFrameRate> IElectraTextureSampleBase::GetFramerate() const
{ return TOptional<FFrameRate>(); }

double IElectraTextureSampleBase::GetAspectRatio() const
{ return 1.0; }

EMediaOrientation IElectraTextureSampleBase::GetOrientation() const
{ return EMediaOrientation::Original; }

bool IElectraTextureSampleBase::IsOutputSrgb() const
{ return true; }

const FMatrix& IElectraTextureSampleBase::GetYUVToRGBMatrix() const
{ return FMatrix::Identity; }

bool IElectraTextureSampleBase::GetFullRange() const
{ return false; }

FMatrix44f IElectraTextureSampleBase::GetSampleToRGBMatrix() const
{ return SampleToRgbMtx; }

const UE::Color::FColorSpace& IElectraTextureSampleBase::GetSourceColorSpace() const
{ return SourceColorSpace; }

UE::Color::EEncoding IElectraTextureSampleBase::GetEncodingType() const
{ return ColorEncoding; }

float IElectraTextureSampleBase::GetHDRNitsNormalizationFactor() const
{ return 1.0f; }

bool IElectraTextureSampleBase::GetDisplayMasteringLuminance(float& OutMin, float& OutMax) const
{ return false; }

TOptional<UE::Color::FColorSpace> IElectraTextureSampleBase::GetDisplayMasteringColorSpace() const
{ return DisplayMasteringColorSpace; }

bool IElectraTextureSampleBase::GetMaxLuminanceLevels(uint16& OutCLL, uint16& OutFALL) const
{ return false; }

MediaShaders::EToneMapMethod IElectraTextureSampleBase::GetToneMapMethod() const
{ return MediaShaders::EToneMapMethod::None; }

float IElectraTextureSampleBase::GetSampleDataScale(bool b10Bit) const
{ return 1.0f; }

#endif
