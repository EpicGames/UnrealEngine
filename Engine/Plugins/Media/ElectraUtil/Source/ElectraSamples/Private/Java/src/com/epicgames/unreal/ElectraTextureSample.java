// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal;

import android.os.Build;
import java.io.IOException;
import java.util.ArrayList;
import android.util.Log;
import android.util.SparseArray;

import android.opengl.*;
import android.graphics.SurfaceTexture;
import android.view.Surface;
import java.nio.ByteBuffer;
import android.media.ImageReader;
import android.media.Image;
import android.hardware.HardwareBuffer;

public class ElectraTextureSample
{
	private static final String TAG = "[ElectraTextureSample]";

	public class FImageResources
	{
		public void Release()
		{
			if (HardwareBuffer != null)
			{
				HardwareBuffer.close();
				HardwareBuffer = null;
			}

			if (Image != null)
			{
				Image.close();
				Image = null;
			}
		}

		public android.media.Image Image = null;
		public android.hardware.HardwareBuffer HardwareBuffer = null;

		public float UScale = 0.0f;
		public float UOffset = 0.0f;
		public float VScale = 0.0f;
		public float VOffset = 0.0f;
	}

	public class FFrameUpdateInfo
	{
		public java.nio.Buffer Buffer;
		public long Timestamp = 0;
		public long Duration = 0;
		public boolean bFrameReady = false;
		public boolean bRegionChanged = false;
		public float UScale = 0.0f;
		public float UOffset = 0.0f;
		public float VScale = 0.0f;
		public float VOffset = 0.0f;
		public int NumPending = 0;
		public FImageResources ImageResources = null;
	}

	private FBitmapRenderer BitmapRenderer = null;

	public ElectraTextureSample()
	{
	}

	public void Initialize(boolean bNewRenderer, boolean bVulkan, long parentHandle)
	{
		if (bNewRenderer)
			BitmapRenderer = new FBitmapRendererNew(bVulkan, parentHandle, 16);
		else
			BitmapRenderer = new FBitmapRendererOld(bVulkan);
	}

	public void Release()
	{
		if (BitmapRenderer != null)
		{
			BitmapRenderer.release();
			BitmapRenderer = null;
		}
	}

	public FFrameUpdateInfo GetVideoFrameUpdateInfo(int destTextureES, int width, int height, boolean bIs10Bbit)
	{
		FFrameUpdateInfo Info = null;
		if (BitmapRenderer != null)
		{
			Info = BitmapRenderer.GetVideoFrameUpdateInfo(destTextureES, width, height, bIs10Bbit);
		}
		return Info;
	}

	public android.view.Surface GetCodecSurface()
	{
		if (BitmapRenderer != null)
		{
			return BitmapRenderer.getSurface();
		}
		return null;
	}

	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/
	/***************************************************************************************************************************************************/

	abstract class FBitmapRenderer
	{
		public abstract void release();
		public abstract FFrameUpdateInfo GetVideoFrameUpdateInfo(int destTexture, int width, int height, boolean bIs10Bit);
		public abstract android.view.Surface getSurface();
	}

	/*
		All this internal surface view does is manage the offscreen bitmap that the media player decoding can render into for eventual extraction to the UE4 buffers.
	*/
	class FBitmapRendererOld extends FBitmapRenderer
	{
		private java.nio.Buffer mFrameData = null;
		private int mFrameDataSize = 0;
		private android.graphics.SurfaceTexture mSurfaceTexture = null;
		private int mTextureWidth = -1;
		private int mTextureHeight = -1;
		private android.view.Surface mSurface = null;
		private int mTextureID = -1;
		private int mFBO = -1;
		private int mBlitVertexShaderID = -1;
		private int[] mBlitFragmentShaderID = new int[2];
		private float[] mTransformMatrix = new float[16];
		private boolean mTriangleVerticesDirty = true;
		private boolean mUseOwnContext = true;
		private boolean mVulkanRenderer = false;

		private int GL_TEXTURE_EXTERNAL_OES = 0x8D65;

		private EGLDisplay mEglDisplay;
		private EGLContext mEglContext;
		private EGLSurface mEglSurface;

		private EGLDisplay mSavedDisplay;
		private EGLContext mSavedContext;
		private EGLSurface mSavedSurfaceDraw;
		private EGLSurface mSavedSurfaceRead;

		private boolean mCreatedEGLDisplay = false;

		public int mNativeDecoderID;

		public FBitmapRendererOld(boolean vulkanRenderer)
		{
			mVulkanRenderer = vulkanRenderer;

			mEglSurface = EGL14.EGL_NO_SURFACE;
			mEglContext = EGL14.EGL_NO_CONTEXT;
			mUseOwnContext = true;

			if (!mVulkanRenderer)
			{
				String RendererString = GLES32.glGetString(GLES32.GL_RENDERER);
				Log.d(TAG,"FBitmapRenderer: GLES: " + RendererString);
				// Do not use shared context if Adreno before 400 or on older Android than Marshmallow
				if (RendererString.contains("Adreno (TM) "))
				{
					java.util.regex.Pattern pattern = java.util.regex.Pattern.compile("Adreno \\(TM\\) (\\d*)");
					java.util.regex.Matcher matcher = pattern.matcher(RendererString);
					if (matcher.find() && matcher.groupCount() >= 1)
					{
						int AdrenoVersion = Integer.parseInt(matcher.group(1));
						if (AdrenoVersion < 400 || android.os.Build.VERSION.SDK_INT < 22)
						{
							GameActivity.Log.debug("VideoDecoder: disabled shared GL context on " + RendererString);
							mUseOwnContext = false;
						}
					}
				}
			}

			if (mUseOwnContext)
			{
				initContext();
				saveContext();
				makeCurrent();
				initSurfaceTexture();
				restoreContext();
			}
			else
			{
				initSurfaceTexture();
			}
		}

		private void initContext()
		{
			mEglDisplay = EGL14.EGL_NO_DISPLAY;
			EGLContext shareContext = EGL14.EGL_NO_CONTEXT;

			int majorver[] = new int[] { 0 };
			int minorver[] = new int[] { 0 };
			if (!mVulkanRenderer)
			{
				mEglDisplay = EGL14.eglGetCurrentDisplay();
				shareContext = EGL14.eglGetCurrentContext();

				if (android.os.Build.VERSION.SDK_INT >= 18 &&
					EGL14.eglQueryContext(mEglDisplay, shareContext, EGLExt.EGL_CONTEXT_MAJOR_VERSION_KHR, majorver, 0) &&
					EGL14.eglQueryContext(mEglDisplay, shareContext, EGLExt.EGL_CONTEXT_MINOR_VERSION_KHR, minorver, 0))
				{
					Log.d(TAG,"VideoDecoder: Existing GL context is version " + majorver[0] + "." + minorver[0]);
				}
				else
				// on some devices eg Galaxy S6, the above fails but we do get EGL14.EGL_CONTEXT_CLIENT_VERSION=3
				if (EGL14.eglQueryContext(mEglDisplay, shareContext, EGL14.EGL_CONTEXT_CLIENT_VERSION, majorver, 0))
				{
					Log.d(TAG, "VideoDecoder: Existing GL context is version " + majorver[0]);
				}
				else
				{
					Log.d(TAG, "VideoDecoder: Existing GL context version not detected");
				}
			}
			else
			{
				mEglDisplay = EGL14.eglGetDisplay(EGL14.EGL_DEFAULT_DISPLAY);
				if (mEglDisplay == EGL14.EGL_NO_DISPLAY)
				{
					Log.e(TAG, "unable to get EGL14 display");
					return;
				}
				int[] version = new int[2];
				if (!EGL14.eglInitialize(mEglDisplay, version, 0, version, 1))
				{
					mEglDisplay = null;
					Log.e(TAG,"unable to initialize EGL14 display");
					return;
				}

				mCreatedEGLDisplay = true;
			}

			int[] configSpec = new int[]
			{
				EGL14.EGL_RENDERABLE_TYPE, EGL14.EGL_OPENGL_ES2_BIT,
				EGL14.EGL_SURFACE_TYPE, EGL14.EGL_PBUFFER_BIT,
				EGL14.EGL_NONE
			};
			EGLConfig[] configs = new EGLConfig[1];
			int[] num_config = new int[1];
			EGL14.eglChooseConfig(mEglDisplay, configSpec, 0, configs, 0, 1, num_config, 0);
			int[] contextAttribsES2 = new int[]
			{
				EGL14.EGL_CONTEXT_CLIENT_VERSION, 2,
				EGL14.EGL_NONE
			};
			int[] contextAttribsES31 = new int[]
			{
				EGLExt.EGL_CONTEXT_MAJOR_VERSION_KHR, 3,
				EGLExt.EGL_CONTEXT_MINOR_VERSION_KHR, 1,
				EGL14.EGL_NONE
			};
			mEglContext = EGL14.eglCreateContext(mEglDisplay, configs[0], shareContext, majorver[0]==3 ? contextAttribsES31 : contextAttribsES2, 0);

			if (EGL14.eglQueryString(mEglDisplay, EGL14.EGL_EXTENSIONS).contains("EGL_KHR_surfaceless_context"))
			{
				mEglSurface = EGL14.EGL_NO_SURFACE;
			}
			else
			{
				int[] pbufferAttribs = new int[]
				{
					EGL14.EGL_NONE
				};
				mEglSurface = EGL14.eglCreatePbufferSurface(mEglDisplay, configs[0], pbufferAttribs, 0);
			}
		}

		private void saveContext()
		{
			mSavedDisplay = EGL14.eglGetCurrentDisplay();
			mSavedContext = EGL14.eglGetCurrentContext();
			mSavedSurfaceDraw = EGL14.eglGetCurrentSurface(EGL14.EGL_DRAW);
			mSavedSurfaceRead = EGL14.eglGetCurrentSurface(EGL14.EGL_READ);
		}

		private void makeCurrent()
		{
			EGL14.eglMakeCurrent(mEglDisplay, mEglSurface, mEglSurface, mEglContext);
		}

		private void restoreContext()
		{
			// When working in a Vulkan environment we will not have a valid display, so just make sure we get rid of the current context
			if (mSavedDisplay == EGL14.EGL_NO_DISPLAY)
			{
				EGL14.eglMakeCurrent(mEglDisplay, EGL14.EGL_NO_SURFACE, EGL14.EGL_NO_SURFACE, EGL14.EGL_NO_CONTEXT);
			}
			else
			{
				EGL14.eglMakeCurrent(mSavedDisplay, mSavedSurfaceDraw, mSavedSurfaceRead, mSavedContext);
			}
		}

		private void initSurfaceTexture()
		{
			int[] textures = new int[1];
			GLES32.glGenTextures(1, textures, 0);
			mTextureID = textures[0];
			if (mTextureID <= 0)
			{
				Log.e(TAG,"mTextureID <= 0");
				release();
				return;
			}
			mSurfaceTexture = new android.graphics.SurfaceTexture(mTextureID);
			mSurface = new android.view.Surface(mSurfaceTexture);

			int[] glInt = new int[1];

			GLES32.glGenFramebuffers(1,glInt,0);
			mFBO = glInt[0];
			if (mFBO <= 0)
			{
				Log.e(TAG, "mFBO <= 0");
				release();
				return;
			}
			// Special shaders for blit of movie texture.
			mBlitVertexShaderID = createShader(GLES32.GL_VERTEX_SHADER, mBlitVextexShader);
			if (mBlitVertexShaderID == 0)
			{
				Log.e(TAG,"mBlitVertexShaderID == 0");
				release();
				return;
			}
			//
			// Android will deliver the data as "image external" to ES.
			// - Anything read from such a texture will "magically" appear as [0..1] range RGB data in the shader
			// - For SDR material this will be sRGB data (still very much with gamma curve applied)
			// - For HDR material thisd will be PQ data (here, too, the EOTF is still very much applied)
			// - CS will be whatever we have in the material playing (mostly Rec.709 (SDR) or Rec.2020 (HDR))
			//
			// -> Hence this indeed complies with the old [0..1] range for "image external". We just need to be very careful
			//    with how to interpret the data.
			//
			// ES:		We copy the data directly into an externally (RHI created) provided texture from where MFW can further process it as needed
			// Vulkan:	We simply copy the data into a CPU-side buffer (any upload into a texture is done in MediaFramework - an ES texture serves as inbetween buffer as we cannot access the "image external" data)
			//
			int[] mBlitFragmentShaderID = new int[2];
			mBlitFragmentShaderID[0] = createShader(GLES32.GL_FRAGMENT_SHADER, mBlitFragmentShaderRGBA_NoConv);
			mBlitFragmentShaderID[1] = createShader(GLES32.GL_FRAGMENT_SHADER, mBlitFragmentShaderBGRA_NoConv);

			if (mBlitFragmentShaderID[0] == 0 || mBlitFragmentShaderID[1] == 0)
			{
				Log.e(TAG,"mBlitFragmentShaderID[i] == 0");
				release();
				return;
			}
			for(int i=0; i<2; ++i)
			{
				mProgram[i] = GLES32.glCreateProgram();
				if (mProgram[i] <= 0)
				{
					Log.e(TAG,"mProgram[i] <= 0");
					release();
					return;
				}
				GLES32.glAttachShader(mProgram[i], mBlitVertexShaderID);
				GLES32.glAttachShader(mProgram[i], mBlitFragmentShaderID[i]);
				GLES32.glLinkProgram(mProgram[i]);
				int[] linkStatus = new int[1];
				GLES32.glGetProgramiv(mProgram[i], GLES32.GL_LINK_STATUS, linkStatus, 0);
				if (linkStatus[0] != GLES32.GL_TRUE)
				{
					Log.e(TAG,"Could not link program: ");
					Log.e(TAG, GLES32.glGetProgramInfoLog(mProgram[i]));
					GLES32.glDeleteProgram(mProgram[i]);
					mProgram[i] = 0;
					release();
					return;
				}
				mPositionAttrib = GLES32.glGetAttribLocation(mProgram[i], "Position");
				mTexCoordsAttrib = GLES32.glGetAttribLocation(mProgram[i], "TexCoords");
				mTextureUniform = GLES32.glGetUniformLocation(mProgram[i], "VideoTexture");
			}

			GLES32.glGenBuffers(1,glInt,0);
			mBlitBuffer = glInt[0];
			if (mBlitBuffer <= 0)
			{
				Log.e(TAG,"mBlitBuffer <= 0");
				release();
				return;
			}

			// Create blit mesh.
			mTriangleVertices = java.nio.ByteBuffer.allocateDirect(
				mTriangleVerticesData.length * FLOAT_SIZE_BYTES)
					.order(java.nio.ByteOrder.nativeOrder()).asFloatBuffer();
			mTriangleVerticesDirty = true;

			// Set up GL state
			if (mUseOwnContext)
			{
				GLES32.glDisable(GLES32.GL_BLEND);
				GLES32.glDisable(GLES32.GL_CULL_FACE);
				GLES32.glDisable(GLES32.GL_SCISSOR_TEST);
				GLES32.glDisable(GLES32.GL_STENCIL_TEST);
				GLES32.glDisable(GLES32.GL_DEPTH_TEST);
				GLES32.glDisable(GLES32.GL_DITHER);
				GLES32.glColorMask(true,true,true,true);
			}
		}

		private void UpdateVertexData()
		{
			if (!mTriangleVerticesDirty || mBlitBuffer <= 0)
			{
				return;
			}

			// fill it in
			mTriangleVertices.position(0);
			mTriangleVertices.put(mTriangleVerticesData).position(0);

			// save VBO state
			int[] glInt = new int[1];
			GLES32.glGetIntegerv(GLES32.GL_ARRAY_BUFFER_BINDING, glInt, 0);
			int previousVBO = glInt[0];

			GLES32.glBindBuffer(GLES32.GL_ARRAY_BUFFER, mBlitBuffer);
			GLES32.glBufferData(GLES32.GL_ARRAY_BUFFER,
				mTriangleVerticesData.length*FLOAT_SIZE_BYTES,
				mTriangleVertices, GLES32.GL_STATIC_DRAW);

			// restore VBO state
			GLES32.glBindBuffer(GLES32.GL_ARRAY_BUFFER, previousVBO);

			mTriangleVerticesDirty = false;
		}

		public boolean isValid()
		{
			return mSurfaceTexture != null;
		}

		private int createShader(int shaderType, String source)
		{
			int shader = GLES32.glCreateShader(shaderType);
			if (shader != 0)
			{
				GLES32.glShaderSource(shader, source);
				GLES32.glCompileShader(shader);
				int[] compiled = new int[1];
				GLES32.glGetShaderiv(shader, GLES32.GL_COMPILE_STATUS, compiled, 0);
				if (compiled[0] == 0)
				{
					Log.e(TAG,"Could not compile shader " + shaderType + ":");
					Log.e(TAG, GLES32.glGetShaderInfoLog(shader));
					GLES32.glDeleteShader(shader);
					shader = 0;
				}
			}
			return shader;
		}

		public android.graphics.SurfaceTexture getSurfaceTexture()
		{
			return mSurfaceTexture;
		}

		public android.view.Surface getSurface()
		{
			return mSurface;
		}

		public int getExternalTextureId()
		{
			return mTextureID;
		}

		private static final int FLOAT_SIZE_BYTES = 4;
		private static final int TRIANGLE_VERTICES_DATA_STRIDE_BYTES = 4 * FLOAT_SIZE_BYTES;
		private static final int TRIANGLE_VERTICES_DATA_POS_OFFSET = 0;
		private static final int TRIANGLE_VERTICES_DATA_UV_OFFSET = 2;
		private float[] mTriangleVerticesData = {
			// X, Y, U, V
			-1.0f, -1.0f, 0.f, 0.f,
			1.0f, -1.0f, 1.f, 0.f,
			-1.0f, 1.0f, 0.f, 1.f,
			1.0f, 1.0f, 1.f, 1.f,
			};

		private java.nio.FloatBuffer mTriangleVertices;

		private final String mBlitVextexShader =
			"attribute vec2 Position;\n" +
			"attribute vec2 TexCoords;\n" +
			"varying vec2 TexCoord;\n" +
			"void main() {\n" +
			"	TexCoord = TexCoords;\n" +
			"	gl_Position = vec4(Position, 0.0, 1.0);\n" +
			"}\n";

		private final String mBlitFragmentShaderBGRA_NoConv =
			"#extension GL_OES_EGL_image_external : require\n" +
			"uniform samplerExternalOES VideoTexture;\n" +
			"varying highp vec2 TexCoord;\n" +
			"void main()\n" +
			"{\n" +
			"	gl_FragColor = texture2D(VideoTexture, TexCoord).bgra;\n" +
			"}\n";
		private final String mBlitFragmentShaderRGBA_NoConv =
			"#extension GL_OES_EGL_image_external : require\n" +
			"uniform samplerExternalOES VideoTexture;\n" +
			"varying highp vec2 TexCoord;\n" +
			"void main()\n" +
			"{\n" +
			"	gl_FragColor = texture2D(VideoTexture, TexCoord).rgba;\n" +
			"}\n";

		private int[] mProgram = new int[2];
		private int mPositionAttrib;
		private int mTexCoordsAttrib;
		private int mBlitBuffer;
		private int mTextureUniform;

		//
		// Vulkan version: resulting image is returned as byte array in frame info data
		//
		public FFrameUpdateInfo updateFrameData(boolean bIs10Bit)
		{
			synchronized(this)
			{
				FFrameUpdateInfo frameUpdateInfo = new FFrameUpdateInfo();
				frameUpdateInfo.bFrameReady = false;
				frameUpdateInfo.bRegionChanged = false;

				int frameDataSize = mTextureWidth*mTextureHeight*4;
				if ((null == mFrameData || frameDataSize != mFrameDataSize) && mTextureWidth > 0 && mTextureHeight > 0)
				{
					mFrameData = java.nio.ByteBuffer.allocateDirect(frameDataSize);
					mFrameDataSize = frameDataSize;
				}
				// Copy surface texture to frame data.
				if (!copyFrameTexture(0, mFrameData, bIs10Bit))
				{
					return null;
				}

				frameUpdateInfo.Buffer = mFrameData;

				long ts = -1;
				SurfaceTexture videoSurfaceTexture = getSurfaceTexture();
				if (videoSurfaceTexture != null)
				{
					ts = videoSurfaceTexture.getTimestamp() / 1000; // nanos to micros
				}
				frameUpdateInfo.Timestamp = ts;
				frameUpdateInfo.bFrameReady = true;
				frameUpdateInfo.bRegionChanged = false;

				return frameUpdateInfo;
			}

		}

		//
		// ES version: resulting image is written to given texture
		//
		public FFrameUpdateInfo updateFrameData(int destTexture, boolean bIs10Bit)
		{
			synchronized(this)
			{
				// Copy surface texture to destination texture.
				if (!copyFrameTexture(destTexture, null, bIs10Bit))
				{
					return null;
				}

				// Get associated frame metadata
				FFrameUpdateInfo frameUpdateInfo = new FFrameUpdateInfo();

				frameUpdateInfo.Buffer = null;

				long ts = -1;
				SurfaceTexture videoSurfaceTexture = getSurfaceTexture();
				if (videoSurfaceTexture != null)
				{
					ts = videoSurfaceTexture.getTimestamp() / 1000; // nanos to micros
				}
				frameUpdateInfo.Timestamp = ts;

				frameUpdateInfo.bFrameReady = true;
				frameUpdateInfo.bRegionChanged = false;

				return frameUpdateInfo;
			}
		}

		// Copy the surface texture to another texture, or to raw data.
		// Note: copying to raw data creates a temporary FBO texture.
		private boolean copyFrameTexture(int destTexture, java.nio.Buffer destData, boolean bIs10Bit)
		{
			if (null == mSurfaceTexture)
			{
				// Can't update if there's no surface to update into.
				return false;
			}

			int[] glInt = new int[1];
			boolean[] glBool = new boolean[1];
			// Either use own context or save states
			boolean previousBlend=false, previousCullFace=false, previousScissorTest=false, previousStencilTest=false, previousDepthTest=false, previousDither=false;
			int previousFBO=0, previousVBO=0, previousMinFilter=0, previousMagFilter=0;
			int[] previousViewport = new int[4];
			if (mUseOwnContext)
			{
				// Received reports of these not being preserved when changing contexts
				GLES32.glActiveTexture(GLES32.GL_TEXTURE0);
				GLES32.glGetTexParameteriv(GLES32.GL_TEXTURE_2D, GLES32.GL_TEXTURE_MIN_FILTER, glInt, 0);
				previousMinFilter = glInt[0];
				GLES32.glGetTexParameteriv(GLES32.GL_TEXTURE_2D, GLES32.GL_TEXTURE_MAG_FILTER, glInt, 0);
				previousMagFilter = glInt[0];

				saveContext();
				makeCurrent();
			}
			else
			{
				// Clear gl errors as they can creep in from the UE4 renderer.
				GLES32.glGetError();

				previousBlend = GLES32.glIsEnabled(GLES32.GL_BLEND);
				previousCullFace = GLES32.glIsEnabled(GLES32.GL_CULL_FACE);
				previousScissorTest = GLES32.glIsEnabled(GLES32.GL_SCISSOR_TEST);
				previousStencilTest = GLES32.glIsEnabled(GLES32.GL_STENCIL_TEST);
				previousDepthTest = GLES32.glIsEnabled(GLES32.GL_DEPTH_TEST);
				previousDither = GLES32.glIsEnabled(GLES32.GL_DITHER);
				GLES32.glGetIntegerv(GLES32.GL_FRAMEBUFFER_BINDING, glInt, 0);
				previousFBO = glInt[0];
				GLES32.glGetIntegerv(GLES32.GL_ARRAY_BUFFER_BINDING, glInt, 0);
				previousVBO = glInt[0];
				GLES32.glGetIntegerv(GLES32.GL_VIEWPORT, previousViewport, 0);

				GLES32.glActiveTexture(GLES32.GL_TEXTURE0);
				GLES32.glGetTexParameteriv(GLES32.GL_TEXTURE_2D, GLES32.GL_TEXTURE_MIN_FILTER, glInt, 0);
				previousMinFilter = glInt[0];
				GLES32.glGetTexParameteriv(GLES32.GL_TEXTURE_2D, GLES32.GL_TEXTURE_MAG_FILTER, glInt, 0);
				previousMagFilter = glInt[0];

				glVerify("save state");
			}

			// Get the latest video texture frame (the surface is really a FIFO of frames received from the decoder when the decoder output is released)
			mSurfaceTexture.updateTexImage();
			mSurfaceTexture.getTransformMatrix(mTransformMatrix);

			float UMin = mTransformMatrix[12];
			float UMax = UMin + mTransformMatrix[0];
			float VMin = mTransformMatrix[13];
			float VMax = VMin + mTransformMatrix[5];

			if (mTriangleVerticesData[2] != UMin ||
				mTriangleVerticesData[6] != UMax ||
				mTriangleVerticesData[11] != VMin ||
				mTriangleVerticesData[3] != VMax)
			{
				mTriangleVerticesData[ 2] = mTriangleVerticesData[10] = UMin;
				mTriangleVerticesData[ 6] = mTriangleVerticesData[14] = UMax;
				mTriangleVerticesData[11] = mTriangleVerticesData[15] = VMin;
				mTriangleVerticesData[ 3] = mTriangleVerticesData[ 7] = VMax;
				mTriangleVerticesDirty = true;
			}

			if (null != destData)
			{
				// Rewind data so that we can write to it.
				destData.position(0);
			}

			if (!mUseOwnContext)
			{
				GLES32.glDisable(GLES32.GL_BLEND);
				GLES32.glDisable(GLES32.GL_CULL_FACE);
				GLES32.glDisable(GLES32.GL_SCISSOR_TEST);
				GLES32.glDisable(GLES32.GL_STENCIL_TEST);
				GLES32.glDisable(GLES32.GL_DEPTH_TEST);
				GLES32.glDisable(GLES32.GL_DITHER);
				GLES32.glColorMask(true,true,true,true);

				glVerify("reset state");
			}

			GLES32.glViewport(0, 0, mTextureWidth, mTextureHeight);

			glVerify("set viewport");

			// Set-up FBO target texture..
			int FBOTextureID = 0;
			if (null != destData)
			{
				// Create temporary FBO for data copy.
				GLES32.glGenTextures(1,glInt,0);
				FBOTextureID = glInt[0];
			}
			else
			{
				// Use the given texture as the FBO.
				FBOTextureID = destTexture;
			}
			// Set the FBO to draw into the texture one-to-one.
			GLES32.glBindTexture(GLES32.GL_TEXTURE_2D, FBOTextureID);
			GLES32.glTexParameteri(GLES32.GL_TEXTURE_2D,
				GLES32.GL_TEXTURE_MIN_FILTER, GLES32.GL_NEAREST);
			GLES32.glTexParameteri(GLES32.GL_TEXTURE_2D,
				GLES32.GL_TEXTURE_MAG_FILTER, GLES32.GL_NEAREST);
			GLES32.glTexParameteri(GLES32.GL_TEXTURE_2D,
				GLES32.GL_TEXTURE_WRAP_S, GLES32.GL_CLAMP_TO_EDGE);
			GLES32.glTexParameteri(GLES32.GL_TEXTURE_2D,
				GLES32.GL_TEXTURE_WRAP_T, GLES32.GL_CLAMP_TO_EDGE);
			// Create the temp FBO data if needed.
			if (null != destData)
			{
				GLES32.glTexImage2D(GLES32.GL_TEXTURE_2D, 0,
					bIs10Bit ? GLES32.GL_RGB10_A2 : GLES32.GL_RGBA,
					mTextureWidth, mTextureHeight,
					0,
					GLES32.GL_RGBA,
					bIs10Bit ? GLES32.GL_UNSIGNED_INT_2_10_10_10_REV : GLES32.GL_UNSIGNED_BYTE, null);
			}

			glVerify("set-up FBO texture");
			// Set to render to the FBO.
			GLES32.glBindFramebuffer(GLES32.GL_FRAMEBUFFER, mFBO);
			glVerify("glBindFramebuffer");
			GLES32.glFramebufferTexture2D(
				GLES32.GL_FRAMEBUFFER,
				GLES32.GL_COLOR_ATTACHMENT0,
				GLES32.GL_TEXTURE_2D, FBOTextureID, 0);

			// check status
			int status = GLES32.glCheckFramebufferStatus(GLES32.GL_FRAMEBUFFER);
			if (status != GLES32.GL_FRAMEBUFFER_COMPLETE)
			{
				Log.w(TAG,"Failed to complete framebuffer attachment ("+status+")");
			}

			// The special shaders to render from the video texture.
			// (Vulkan SDR needs to convert BGR / RGB)
			GLES32.glUseProgram((!bIs10Bit && mVulkanRenderer) ? mProgram[1] : mProgram[0]);

			// Set the mesh that renders the video texture.
			UpdateVertexData();
			GLES32.glBindBuffer(GLES32.GL_ARRAY_BUFFER, mBlitBuffer);
			GLES32.glEnableVertexAttribArray(mPositionAttrib);
			GLES32.glVertexAttribPointer(mPositionAttrib, 2, GLES32.GL_FLOAT, false,
				TRIANGLE_VERTICES_DATA_STRIDE_BYTES, 0);
			GLES32.glEnableVertexAttribArray(mTexCoordsAttrib);
			GLES32.glVertexAttribPointer(mTexCoordsAttrib, 2, GLES32.GL_FLOAT, false,
				TRIANGLE_VERTICES_DATA_STRIDE_BYTES,
				TRIANGLE_VERTICES_DATA_UV_OFFSET*FLOAT_SIZE_BYTES);

			glVerify("setup movie texture read");

//GLES32.glClearColor(1.0f, 1.0f, 1.0f, 1.0f);
//GLES32.glClear( GLES32.GL_COLOR_BUFFER_BIT);

			// connect 'VideoTexture' to video source texture (mTextureID).
			// mTextureID is bound to GL_TEXTURE_EXTERNAL_OES in updateTexImage
			GLES32.glUniform1i(mTextureUniform, 0);
			GLES32.glActiveTexture(GLES32.GL_TEXTURE0);
			GLES32.glBindTexture(GL_TEXTURE_EXTERNAL_OES, mTextureID);

			glVerify("bind FBO movie texture read");

			// Draw the video texture mesh.
			GLES32.glDrawArrays(GLES32.GL_TRIANGLE_STRIP, 0, 4);

			// Make sure GL issues commands to GPU
			GLES32.glFlush();

			glVerify("movie texture read flushed");

			// Read the FBO texture pixels into raw data.
			if (null != destData)
			{
				GLES32.glReadPixels(
					0, 0, mTextureWidth, mTextureHeight,
					GLES32.GL_RGBA,
					bIs10Bit ? GLES32.GL_UNSIGNED_INT_2_10_10_10_REV : GLES32.GL_UNSIGNED_BYTE,
					destData);
			}

			glVerify("draw & read movie texture");

			// Restore state and cleanup.
			if (mUseOwnContext)
			{
				GLES32.glFramebufferTexture2D(
					GLES32.GL_FRAMEBUFFER,
					GLES32.GL_COLOR_ATTACHMENT0,
					GLES32.GL_TEXTURE_2D, 0, 0);

				if (null != destData && FBOTextureID > 0)
				{
					glInt[0] = FBOTextureID;
					GLES32.glDeleteTextures(1, glInt, 0);
				}

				restoreContext();

				// Restore previous texture filtering
				GLES32.glTexParameteri(GLES32.GL_TEXTURE_2D, GLES32.GL_TEXTURE_MIN_FILTER, previousMinFilter);
				GLES32.glTexParameteri(GLES32.GL_TEXTURE_2D, GLES32.GL_TEXTURE_MAG_FILTER, previousMagFilter);
			}
			else
			{
				GLES32.glBindFramebuffer(GLES32.GL_FRAMEBUFFER, previousFBO);
				if (null != destData && FBOTextureID > 0)
				{
					glInt[0] = FBOTextureID;
					GLES32.glDeleteTextures(1, glInt, 0);
				}
				GLES32.glBindBuffer(GLES32.GL_ARRAY_BUFFER, previousVBO);

				GLES32.glViewport(previousViewport[0], previousViewport[1],	previousViewport[2], previousViewport[3]);
				if (previousBlend) GLES32.glEnable(GLES32.GL_BLEND);
				if (previousCullFace) GLES32.glEnable(GLES32.GL_CULL_FACE);
				if (previousScissorTest) GLES32.glEnable(GLES32.GL_SCISSOR_TEST);
				if (previousStencilTest) GLES32.glEnable(GLES32.GL_STENCIL_TEST);
				if (previousDepthTest) GLES32.glEnable(GLES32.GL_DEPTH_TEST);
				if (previousDither) GLES32.glEnable(GLES32.GL_DITHER);

				GLES32.glTexParameteri(GLES32.GL_TEXTURE_2D, GLES32.GL_TEXTURE_MIN_FILTER, previousMinFilter);
				GLES32.glTexParameteri(GLES32.GL_TEXTURE_2D, GLES32.GL_TEXTURE_MAG_FILTER, previousMagFilter);

				// invalidate cached state in RHI
				GLES32.glDisableVertexAttribArray(mPositionAttrib);
				GLES32.glDisableVertexAttribArray(mTexCoordsAttrib);
				nativeClearCachedAttributeState(mPositionAttrib, mTexCoordsAttrib); // offered by RHI for ES on Android
			}

			return true;
		}

		private void showGlError(String op, int error)
		{
			switch (error)
			{
				case GLES32.GL_INVALID_ENUM:						Log.e(TAG,"MediaPlayer$BitmapRenderer: " + op + ": glGetError GL_INVALID_ENUM");  break;
				case GLES32.GL_INVALID_OPERATION:					Log.e(TAG,"MediaPlayer$BitmapRenderer: " + op + ": glGetError GL_INVALID_OPERATION");  break;
				case GLES32.GL_INVALID_FRAMEBUFFER_OPERATION:		Log.e(TAG,"MediaPlayer$BitmapRenderer: " + op + ": glGetError GL_INVALID_FRAMEBUFFER_OPERATION");  break;
				case GLES32.GL_INVALID_VALUE:						Log.e(TAG,"MediaPlayer$BitmapRenderer: " + op + ": glGetError GL_INVALID_VALUE");  break;
				case GLES32.GL_FRAMEBUFFER_INCOMPLETE_ATTACHMENT:	Log.e(TAG,"MediaPlayer$BitmapRenderer: " + op + ": glGetError GL_FRAMEBUFFER_INCOMPLETE_ATTACHMENT");  break;
				case GLES32.GL_FRAMEBUFFER_INCOMPLETE_DIMENSIONS:	Log.e(TAG,"MediaPlayer$BitmapRenderer: " + op + ": glGetError GL_FRAMEBUFFER_INCOMPLETE_DIMENSIONS");  break;
				case GLES32.GL_FRAMEBUFFER_UNSUPPORTED:				Log.e(TAG,"MediaPlayer$BitmapRenderer: " + op + ": glGetError GL_FRAMEBUFFER_UNSUPPORTED");  break;
				case GLES32.GL_OUT_OF_MEMORY:						Log.e(TAG,"MediaPlayer$BitmapRenderer: " + op + ": glGetError GL_OUT_OF_MEMORY");  break;
				default:											Log.e(TAG,"MediaPlayer$BitmapRenderer: " + op + ": glGetError " + error);
			}
		}

		private void glVerify(String op)
		{
			int error;
			while ((error = GLES32.glGetError()) != GLES32.GL_NO_ERROR)
			{
				showGlError(op, error);
				throw new RuntimeException(op + ": glGetError " + error);
			}
		}

		private void glWarn(String op)
		{
			int error;
			while ((error = GLES32.glGetError()) != GLES32.GL_NO_ERROR)
			{
				showGlError(op, error);
			}
		}

		public FFrameUpdateInfo GetVideoFrameUpdateInfo(int destTexture, int width, int height, boolean bIs10Bit)
		{
			mTextureWidth = width;
			mTextureHeight = height;

			if (mVulkanRenderer)
			{
				// Return data as byte buffer
				return updateFrameData(bIs10Bit);
			}
			// Return data in ES texture
			return updateFrameData(destTexture, bIs10Bit);
		}

		public void release()
		{
			synchronized(this)
			{
				if (mSurfaceTexture != null)
				{
					mSurfaceTexture.setOnFrameAvailableListener(null);
				}
			}

			if (mEglContext != EGL14.EGL_NO_CONTEXT)
			{
				saveContext();
				makeCurrent();
			}
			if (null != mSurface)
			{
				mSurface.release();
				mSurface = null;
			}
			if (null != mSurfaceTexture)
			{
				mSurfaceTexture.release();
				mSurfaceTexture = null;
			}
			int[] glInt = new int[1];
			if (mBlitBuffer > 0)
			{
				glInt[0] = mBlitBuffer;
				GLES32.glDeleteBuffers(1,glInt,0);
				mBlitBuffer = -1;
			}
			for(int i=0; i<2; ++i)
			{
				if (mProgram[i] > 0)
				{
					GLES32.glDeleteProgram(mProgram[i]);
					mProgram[i] = -1;
				}
			}
			if (mBlitVertexShaderID > 0)
			{
				GLES32.glDeleteShader(mBlitVertexShaderID);
				mBlitVertexShaderID = -1;
			}
			for(int i=0; i<2; ++i)
			{
				if (mBlitFragmentShaderID[i] > 0)
				{
					GLES32.glDeleteShader(mBlitFragmentShaderID[i]);
					mBlitFragmentShaderID[i] = -1;
				}
			}
			if (mFBO > 0)
			{
				glInt[0] = mFBO;
				GLES32.glDeleteFramebuffers(1,glInt,0);
				mFBO = -1;
			}
			if (mTextureID > 0)
			{
				glInt[0] = mTextureID;
				GLES32.glDeleteTextures(1,glInt,0);
				mTextureID = -1;
			}
			if (mEglSurface != EGL14.EGL_NO_SURFACE)
			{
				EGL14.eglDestroySurface(mEglDisplay, mEglSurface);
				mEglSurface = EGL14.EGL_NO_SURFACE;
			}
			if (mEglContext != EGL14.EGL_NO_CONTEXT)
			{
				restoreContext();
				EGL14.eglDestroyContext(mEglDisplay, mEglContext);
				mEglContext = EGL14.EGL_NO_CONTEXT;
			}
			if (mCreatedEGLDisplay)
			{
				EGL14.eglTerminate(mEglDisplay);
				mEglDisplay = EGL14.EGL_NO_DISPLAY;
				mCreatedEGLDisplay = false;
			}
		}
	}

	/*
		All this internal surface view does is manage the offscreen bitmap that the media player decoding can render into for eventual extraction to the UE4 buffers.
	*/
	class FBitmapRendererNew extends FBitmapRenderer implements ImageReader.OnImageAvailableListener
	{
		private java.nio.Buffer mFrameData = null;
		private int mFrameDataSize = 0;
		private android.media.ImageReader mImageReader = null;
		private android.os.Handler mImageReaderHandler = null;
		private int mTextureWidth = -1;
		private int mTextureHeight = -1;
		private android.view.Surface mSurface = null;
		private boolean mUseOwnContext = true;
		private boolean mVulkanRenderer = false;
		private long mParentHandle = 0;

		public int mNativeDecoderID;

		public FBitmapRendererNew(boolean vulkanRenderer, long parentHandle, int maxQueueLength)
		{
			mVulkanRenderer = vulkanRenderer;
			mParentHandle = parentHandle;

			mImageReader = android.media.ImageReader.newInstance(1, 1, android.graphics.ImageFormat.YUV_420_888, maxQueueLength, android.hardware.HardwareBuffer.USAGE_GPU_SAMPLED_IMAGE);

			mSurface = mImageReader.getSurface();

			mImageReaderHandler = new android.os.Handler(android.os.Looper.getMainLooper());
			mImageReader.setOnImageAvailableListener(this, mImageReaderHandler);
		}

		public void onImageAvailable(ImageReader reader)
		{
			nativeSignalSurfaceReadEvent(mParentHandle);
		}

		public boolean isValid()
		{
			return mImageReader != null;
		}

		public android.view.Surface getSurface()
		{
			return mSurface;
		}

		public FFrameUpdateInfo GetVideoFrameUpdateInfo(int destTexture, int width, int height, boolean bIs10Bit)
		{
			mTextureWidth = width;
			mTextureHeight = height;

			return updateFrameData(bIs10Bit);
		}

		public FFrameUpdateInfo updateFrameData(boolean bIs10Bit)
		{
			synchronized(this)
			{
				// Copy surface texture to destination texture.
				FImageResources ImageResources = getFrameTextureInfo();
				if (ImageResources == null)
				{
					return null;
				}

				// Get associated frame metadata
				FFrameUpdateInfo frameUpdateInfo = new FFrameUpdateInfo();

				frameUpdateInfo.ImageResources = ImageResources;

				frameUpdateInfo.Timestamp = -1;
				frameUpdateInfo.bFrameReady = true;
				frameUpdateInfo.bRegionChanged = false;

				return frameUpdateInfo;
			}
		}


		private FImageResources getFrameTextureInfo()
		{
			if (null == mImageReader)
			{
				// Can't update if there's no surface to update into.
				return null;
			}

			android.media.Image Image = null;
			try
			{
				// Get image data from Surface / Queue...
				Image = mImageReader.acquireNextImage();
			} 
			catch (Exception e) 
			{
				Log.d(TAG, String.format("getFrameTextureInfo: ImageReader.acquireNextImage() threw: %s", e.toString()));
				return null;
			}

			if (Image == null)
			{
				Log.d(TAG, "getFrameTextureInfo requested, but no images queued...");
				return null;
			}

			// Prepare to capture any resources we need to track for the output image
			FImageResources ImageResources = new FImageResources();
			ImageResources.Image = Image;

			ImageResources.HardwareBuffer = ImageResources.Image.getHardwareBuffer();
			if (ImageResources.HardwareBuffer == null)
			{
				Log.d(TAG, "Could not get HardwareBuffer from Image!");
				return null;
			}

			android.graphics.Rect CropRect = ImageResources.Image.getCropRect();
			ImageResources.UScale = (float)(CropRect.right - CropRect.left) / mTextureWidth;
			ImageResources.UOffset = (float)CropRect.left / mTextureWidth;
			ImageResources.VScale = (float)(CropRect.bottom - CropRect.top) / mTextureHeight;
			ImageResources.VOffset = (float)CropRect.top / mTextureHeight;

			return ImageResources;
		}

		public FFrameUpdateInfo GetVideoFrameUpdateInfo(int width, int height, boolean bIs10Bit)
		{
			mTextureWidth = width;
			mTextureHeight = height;

			return updateFrameData(bIs10Bit);
		}

		public void release()
		{
			synchronized(this)
			{
				if (mImageReader != null)
				{
					mImageReader.setOnImageAvailableListener(null, null);
				}
			}

			if (null != mSurface)
			{
				mSurface.release();
				mSurface = null;
			}
			if (mImageReader != null)
			{
				mImageReader.close();
				mImageReader = null;
			}
		}
	}

	public native void nativeClearCachedAttributeState(int PositionAttrib, int TexCoordsAttrib);
	public native void nativeSignalSurfaceReadEvent(long parentHandle);
}


