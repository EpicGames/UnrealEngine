// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

#include "ElectraTextureSample.h"

FElectraTextureSample::~FElectraTextureSample()
{
	ShutdownPoolable();
}

uint32 FElectraTextureSample::GetStride() const
{
	return Stride;
}

const void* FElectraTextureSample::GetBuffer()
{
	return Buffer ? Buffer->GetData() : nullptr;
}

EMediaTextureSampleFormat FElectraTextureSample::GetFormat() const
{
	return SampleFormat;
}

FRHITexture* FElectraTextureSample::GetTexture() const
{
	return nullptr;
}

IMediaTextureSampleConverter* FElectraTextureSample::GetMediaTextureSampleConverter()
{
	return nullptr;
}

#if !UE_SERVER
void FElectraTextureSample::ShutdownPoolable()
{
	if (bWasShutDown)
	{
		return;
	}
	Buffer.Reset();
	IElectraTextureSampleBase::ShutdownPoolable();
}
#endif

bool FElectraTextureSample::FinishInitialization()
{
	switch(PixelFormat)
	{
		case PF_NV12:
		{
			SampleFormat = EMediaTextureSampleFormat::CharNV12;
			break;
		}
		case PF_P010:
		{
			SampleFormat = EMediaTextureSampleFormat::P010;
			break;
		}
		case PF_DXT1:
		{
			SampleFormat = EMediaTextureSampleFormat::DXT1;
			break;
		}
		case PF_DXT5:
		{
			switch(PixelFormatEncoding)
			{
				case EElectraTextureSamplePixelEncoding::YCoCg:
				{
					SampleFormat = EMediaTextureSampleFormat::YCoCg_DXT5;
					break;
				}
				case EElectraTextureSamplePixelEncoding::YCoCg_Alpha:
				{
					SampleFormat = EMediaTextureSampleFormat::YCoCg_DXT5_Alpha_BC4;
					break;
				}
				case EElectraTextureSamplePixelEncoding::Native:
				{
					SampleFormat = EMediaTextureSampleFormat::DXT5;
					break;
				}
				default:
				{
					SampleFormat = EMediaTextureSampleFormat::Undefined;
					return false;
				}
			}
			break;
		}
		case PF_BC4:
		{
			SampleFormat = EMediaTextureSampleFormat::BC4;
			break;
		}
		case PF_A16B16G16R16:
		{
			switch(PixelFormatEncoding)
			{
				case EElectraTextureSamplePixelEncoding::CbY0CrY1:
				{
					SampleFormat = EMediaTextureSampleFormat::YUVv216;
					break;
				}
				case EElectraTextureSamplePixelEncoding::Y0CbY1Cr:
				{
					// TODO!!!!!!!! ("swapped" v216 - seems there is no real format for this?)
					SampleFormat = EMediaTextureSampleFormat::Undefined;
					return false;
				}
				case EElectraTextureSamplePixelEncoding::YCbCr_Alpha:
				{
					SampleFormat = EMediaTextureSampleFormat::Y416;
					break;
				}
				case EElectraTextureSamplePixelEncoding::ARGB_BigEndian:
				{
					SampleFormat = EMediaTextureSampleFormat::ARGB16_BIG;
					break;
				}
				case EElectraTextureSamplePixelEncoding::Native:
				{
					SampleFormat = EMediaTextureSampleFormat::ABGR16;
					break;
				}
				default:
				{
					SampleFormat = EMediaTextureSampleFormat::Undefined;
					return false;
				}
			}
			break;
		}
		case PF_R16G16B16A16_UNORM:
		{
			if (PixelFormatEncoding != EElectraTextureSamplePixelEncoding::Native)
			{
				SampleFormat = EMediaTextureSampleFormat::Undefined;
				return false;
			}
			SampleFormat = EMediaTextureSampleFormat::RGBA16;
			break;
		}
		case PF_A32B32G32R32F:
		{
			switch(PixelFormatEncoding)
			{
				case EElectraTextureSamplePixelEncoding::Native:
				{
					SampleFormat = EMediaTextureSampleFormat::FloatRGBA;
					break;
				}
				case EElectraTextureSamplePixelEncoding::YCbCr_Alpha:
				{
					SampleFormat = EMediaTextureSampleFormat::R4FL;
					break;
				}
				default:
				{
					SampleFormat = EMediaTextureSampleFormat::Undefined;
					return false;
				}
			}
			break;
		}
		case PF_B8G8R8A8:
		{
			switch(PixelFormatEncoding)
			{
				case EElectraTextureSamplePixelEncoding::CbY0CrY1:
				{
					SampleFormat = EMediaTextureSampleFormat::Char2VUY;
					break;
				}
				case EElectraTextureSamplePixelEncoding::Y0CbY1Cr:
				{
					SampleFormat = EMediaTextureSampleFormat::CharYUY2;
					break;
				}
				case EElectraTextureSamplePixelEncoding::YCbCr_Alpha:
				{
					SampleFormat = EMediaTextureSampleFormat::CharAYUV;
					break;
				}
				case EElectraTextureSamplePixelEncoding::Native:
				{
					SampleFormat = EMediaTextureSampleFormat::CharBGRA;
					break;
				}
				default:
				{
					SampleFormat = EMediaTextureSampleFormat::Undefined;
					return false;
				}
			}
			break;
		}
		case PF_R8G8B8A8:
		{
			switch(PixelFormatEncoding)
			{
				case EElectraTextureSamplePixelEncoding::Native:
				{
					SampleFormat = EMediaTextureSampleFormat::CharRGBA;
					break;
				}
				default:
				{
					SampleFormat = EMediaTextureSampleFormat::Undefined;
					return false;
				}
			}
			break;
		}
		case PF_A2B10G10R10:
		{
			switch(PixelFormatEncoding)
			{
				case EElectraTextureSamplePixelEncoding::CbY0CrY1:
				{
					SampleFormat = EMediaTextureSampleFormat::YUVv210;
					break;
				}
				case EElectraTextureSamplePixelEncoding::Native:
				{
					SampleFormat = EMediaTextureSampleFormat::CharBGR10A2;
					break;
				}
				default:
				{
					SampleFormat = EMediaTextureSampleFormat::Undefined;
					return false;
				}
			}
			break;
		}
		default:
		{
			check(!"Decoder sample format not supported in Electra texture sample!");
			return false;
		}
	}

	bCanUseSRGB = (SampleFormat == EMediaTextureSampleFormat::CharBGRA ||
				   SampleFormat == EMediaTextureSampleFormat::CharRGBA ||
				   SampleFormat == EMediaTextureSampleFormat::CharBMP ||
				   SampleFormat == EMediaTextureSampleFormat::DXT1 ||
				   SampleFormat == EMediaTextureSampleFormat::DXT5);

	return IElectraTextureSampleBase::FinishInitialization();
}
