// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

#if PLATFORM_WINDOWS

#include "ElectraTextureSample.h"

#if !UE_SERVER
#include "ElectraSamplesModule.h"

#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RenderUtils.h"

#include "ID3D12DynamicRHI.h"
#ifdef ELECTRA_HAVE_DX11
#include "ID3D11DynamicRHI.h"
#endif

#include "Async/Async.h"


#include COMPILED_PLATFORM_HEADER(ElectraTextureSampleGPUBufferHelper.h)

/*
	Short summary of how we get data:

	- Win10+ (HW decode is used at all times if handling H.264/5)
	-- DX11:   we receive data in GPU space as NV12/P010 texture
	-- DX12:   we receive data in CPU(yes) space as NV12/P010 texture -=UNLESS=- we have SDK 22621+ and suitable runtime support -> DX12 texture in GPU space
	-- Vulkan: we receive data in CPU(yes) space as NV12/P010 texture
	-- Other codec's data usually arrives as CPU space texture buffer

	- Win8:
	-- SW-decode fall back: we receive data in a shared DX11 texture (despite it being SW decode) in NV12 format

	- Win7:
	-- we receive data in a CPU space buffer in NV12 format (no P010 support)
*/

// -------------------------------------------------------------------------------------------------------------------------

DECLARE_GPU_STAT_NAMED(MediaWinDecoder_Convert, TEXT("MediaWinDecoder_Convert"));

// --------------------------------------------------------------------------------------------------------------------------


FElectraTextureSample::~FElectraTextureSample()
{
	ShutdownPoolable();
	// Release DX11 resources asynchronously. There is some internal synchronization going on that takes a while
	// to finish that would block the game thread.
	if (GIsRunning && (TextureDX11.IsValid() || D3D11Device.IsValid()))
	{
		struct FRefCountedObjs
		{
			~FRefCountedObjs()
			{
				TextureDX11.SafeRelease();
				D3D11Device.SafeRelease();
			}
			TRefCountPtr<ID3D11Texture2D> TextureDX11;
			TRefCountPtr<ID3D11Device> D3D11Device;
		};
		FRefCountedObjs* rco = new FRefCountedObjs;
		rco->TextureDX11 = MoveTemp(TextureDX11);
		rco->D3D11Device = MoveTemp(D3D11Device);
		Async(EAsyncExecution::TaskGraph, [rco]()
		{
			delete rco;
		});
	}
}


void FElectraTextureSample::ClearDX11Vars()
{
	TextureDX11 = nullptr;
	D3D11Device = nullptr;
}

void FElectraTextureSample::ClearDX12Vars()
{
	TextureDX12 = nullptr;
	TextureDX12Dim.X = TextureDX12Dim.Y = 0;
	D3DFence = nullptr;
	FenceValue = 0;
}

void FElectraTextureSample::ClearBufferVars()
{
	Buffer.Reset();
}

bool FElectraTextureSample::FinishInitialization()
{
	switch(PixelFormat)
	{
		case PF_NV12:
		{
			SampleFormat = EMediaTextureSampleFormat::CharNV12;
			break;
		}
		case PF_P010:
		{
			SampleFormat = EMediaTextureSampleFormat::P010;
			break;
		}
		case PF_DXT1:
		{
			SampleFormat = EMediaTextureSampleFormat::DXT1;
			break;
		}
		case PF_DXT5:
		{
			switch(PixelFormatEncoding)
			{
				case EElectraTextureSamplePixelEncoding::YCoCg:
				{
					SampleFormat = EMediaTextureSampleFormat::YCoCg_DXT5;
					break;
				}
				case EElectraTextureSamplePixelEncoding::YCoCg_Alpha:
				{
					SampleFormat = EMediaTextureSampleFormat::YCoCg_DXT5_Alpha_BC4;
					break;
				}
				case EElectraTextureSamplePixelEncoding::Native:
				{
					SampleFormat = EMediaTextureSampleFormat::DXT5;
					break;
				}
				default:
				{
					SampleFormat = EMediaTextureSampleFormat::Undefined;
					return false;
				}
			}
			break;
		}
		case PF_BC4:
		{
			SampleFormat = EMediaTextureSampleFormat::BC4;
			break;
		}
		case PF_A16B16G16R16:
		{
			switch(PixelFormatEncoding)
			{
				case EElectraTextureSamplePixelEncoding::CbY0CrY1:
				{
					SampleFormat = EMediaTextureSampleFormat::YUVv216;
					break;
				}
				case EElectraTextureSamplePixelEncoding::Y0CbY1Cr:
				{
					SampleFormat = EMediaTextureSampleFormat::ShortYUY2;
					break;
				}
				case EElectraTextureSamplePixelEncoding::YCbCr_Alpha:
				{
					SampleFormat = EMediaTextureSampleFormat::Y416;
					break;
				}
				case EElectraTextureSamplePixelEncoding::ARGB_BigEndian:
				{
					SampleFormat = EMediaTextureSampleFormat::ARGB16_BIG;
					break;
				}
				case EElectraTextureSamplePixelEncoding::Native:
				{
					SampleFormat = EMediaTextureSampleFormat::ABGR16;
					break;
				}
				default:
				{
					SampleFormat = EMediaTextureSampleFormat::Undefined;
					return false;
				}
			}
			break;
		}
		case PF_R16G16B16A16_UNORM:
		{
			if (PixelFormatEncoding != EElectraTextureSamplePixelEncoding::Native)
			{
				SampleFormat = EMediaTextureSampleFormat::Undefined;
				return false;
			}
			SampleFormat = EMediaTextureSampleFormat::RGBA16;
			break;
		}
		case PF_A32B32G32R32F:
		{
			switch(PixelFormatEncoding)
			{
				case EElectraTextureSamplePixelEncoding::Native:
				{
					SampleFormat = EMediaTextureSampleFormat::FloatRGBA;
					break;
				}
				case EElectraTextureSamplePixelEncoding::YCbCr_Alpha:
				{
					SampleFormat = EMediaTextureSampleFormat::R4FL;
					break;
				}
				default:
				{
					SampleFormat = EMediaTextureSampleFormat::Undefined;
					return false;
				}
			}
			break;
		}
		case PF_B8G8R8A8:
		{
			switch(PixelFormatEncoding)
			{
				case EElectraTextureSamplePixelEncoding::CbY0CrY1:
				{
					SampleFormat = EMediaTextureSampleFormat::Char2VUY;
					break;
				}
				case EElectraTextureSamplePixelEncoding::Y0CbY1Cr:
				{
					SampleFormat = EMediaTextureSampleFormat::CharYUY2;
					break;
				}
				case EElectraTextureSamplePixelEncoding::YCbCr_Alpha:
				{
					SampleFormat = EMediaTextureSampleFormat::CharAYUV;
					break;
				}
				case EElectraTextureSamplePixelEncoding::Native:
				{
					SampleFormat = EMediaTextureSampleFormat::CharBGRA;
					break;
				}
				default:
				{
					SampleFormat = EMediaTextureSampleFormat::Undefined;
					return false;
				}
			}
			break;
		}
		case PF_R8G8B8A8:
		{
			switch(PixelFormatEncoding)
			{
				case EElectraTextureSamplePixelEncoding::Native:
				{
					SampleFormat = EMediaTextureSampleFormat::CharRGBA;
					break;
				}
				default:
				{
					SampleFormat = EMediaTextureSampleFormat::Undefined;
					return false;
				}
			}
			break;
		}
		case PF_A2B10G10R10:
		{
			switch(PixelFormatEncoding)
			{
				case EElectraTextureSamplePixelEncoding::CbY0CrY1:
				{
					SampleFormat = EMediaTextureSampleFormat::YUVv210;
					break;
				}
				case EElectraTextureSamplePixelEncoding::Native:
				{
					SampleFormat = EMediaTextureSampleFormat::CharBGR10A2;
					break;
				}
				default:
				{
					SampleFormat = EMediaTextureSampleFormat::Undefined;
					return false;
				}
			}
			break;
		}
		default:
		{
			check(!"Decoder sample format not supported in Electra texture sample!");
			return false;
		}
	}

	bCanUseSRGB = (SampleFormat == EMediaTextureSampleFormat::CharBGRA ||
				   SampleFormat == EMediaTextureSampleFormat::CharRGBA ||
				   SampleFormat == EMediaTextureSampleFormat::CharBMP ||
				   SampleFormat == EMediaTextureSampleFormat::DXT1 ||
				   SampleFormat == EMediaTextureSampleFormat::DXT5);

	return IElectraTextureSampleBase::FinishInitialization();
}


#if !UE_SERVER
void FElectraTextureSample::ShutdownPoolable()
{
	if (bWasShutDown)
	{
		return;
	}
	// If we use DX12 with texture data, we use the RHI texture only to refer to the actual "native" D3D resource we created -> get rid of this "alias" as soon as we retire to the pool
	if (TextureDX12 && RHIGetInterfaceType() == ERHIInterfaceType::D3D12)
	{
		Texture = nullptr;
	}

	// Drop fence (if any)
	D3DFence = nullptr;

	// Drop decoder output resource
	DecoderOutputResource = nullptr;

	// Reset the CPU buffer (if any)
	Buffer.Reset();

	// Shared DX11 texture needs further consideration
	if (SourceType == ESourceType::SharedTextureDX11)
	{
		// Correctly release the keyed mutex when the sample is returned to the pool
		TRefCountPtr<IDXGIResource> OtherResource(nullptr);
		if (TextureDX11)
		{
			TextureDX11->QueryInterface(__uuidof(IDXGIResource), (void**)&OtherResource);
		}
		// Make sure DX11 shared texture sync state is as expected
		if (OtherResource)
		{
			HANDLE SharedHandle = nullptr;
			if (SUCCEEDED(OtherResource->GetSharedHandle(&SharedHandle)))
			{
				TRefCountPtr<ID3D11Resource> SharedResource;
				D3D11Device->OpenSharedResource(SharedHandle, __uuidof(ID3D11Texture2D), (void**)&SharedResource);
				if (SharedResource)
				{
					TRefCountPtr<IDXGIKeyedMutex> KeyedMutex;
					OtherResource->QueryInterface(_uuidof(IDXGIKeyedMutex), (void**)&KeyedMutex);
					if (KeyedMutex)
					{
						// Reset keyed mutex. NOTE: Do NOT use the SUCCEEDED() macro here since timeouts (a positive value) count as successful!
						if (KeyedMutex->AcquireSync(1, 0) == S_OK)
						{
							// Texture was never read
							KeyedMutex->ReleaseSync(0);
						}
						else if (KeyedMutex->AcquireSync(2, 0) == S_OK)
						{
							// Texture was read at least once
							KeyedMutex->ReleaseSync(0);
						}
					}
				}
			}
		}
	}

	IElectraTextureSampleBase::ShutdownPoolable();
}

bool FElectraTextureSample::IsReadyForReuse()
{
	// Make sure any async copies to any internal GPU resources are done (DX12 case)
	// (this also guards any decoder created resources we keep to read from)
	return TextureDX12.IsValid() && D3DFence.IsValid() ? D3DFence->GetCompletedValue() >= FenceValue : true;
}

#endif


#if WITH_ENGINE
FRHITexture* FElectraTextureSample::GetTexture() const
{
	check(IsInRenderingThread());

	// Is this DX12?
	if (RHIGetInterfaceType() == ERHIInterfaceType::D3D12)
	{
		// Yes, see if we have a real D3D resource...
		if (TRefCountPtr<IUnknown> TextureCommon = GetTextureD3D())
		{
			// Yes. Do we need a new RHI texture?
			if (!Texture.IsValid() || Texture->GetSizeX() != SampleDim.X || Texture->GetSizeY() != SampleDim.Y)
			{
				// Yes...
				TRefCountPtr<ID3D12Resource> TextureCommonDX12;
				HRESULT Result = TextureCommon->QueryInterface(__uuidof(ID3D12Resource), (void**)TextureCommonDX12.GetInitReference());

				if (FAILED(Result))
				{
					// Support shared dxgi resource.
					TRefCountPtr<IDXGIResource> DxgiResource;
					Result = TextureCommon->QueryInterface(__uuidof(IDXGIResource), (void**)DxgiResource.GetInitReference());
					if (SUCCEEDED(Result))
					{
						HANDLE SharedHandle;
						Result = DxgiResource->GetSharedHandle(&SharedHandle);
						if (SUCCEEDED(Result))
						{
							Result = GetID3D12DynamicRHI()->RHIGetDevice_NoMGPU()->OpenSharedHandle(SharedHandle, __uuidof(ID3D12Resource), (void**)TextureCommonDX12.GetInitReference());
						}
					}
				}

				if (ensure(SUCCEEDED(Result)))
				{
					// Setup a suitable RHI texture (it will also become an additional owner of the data)
					ETextureCreateFlags Flags = ETextureCreateFlags::ShaderResource;
					if (IsOutputSrgb() && bCanUseSRGB)
					{
						Flags = Flags | ETextureCreateFlags::SRGB;
					}
					Texture = static_cast<ID3D12DynamicRHI*>(GDynamicRHI->GetNonValidationRHI())->RHICreateTexture2DFromResource(GetPixelFormat(), Flags, FClearValueBinding(), TextureCommonDX12);
				}
			}
		}
	}

	return Texture;
}
#endif //WITH_ENGINE


IMediaTextureSampleConverter* FElectraTextureSample::GetMediaTextureSampleConverter()
{
	const bool bHaveTexture = TextureDX11 || TextureDX12;

	// DXT5 & BC4 combo-data in CPU side buffer?
	if (!bHaveTexture && SampleFormat == EMediaTextureSampleFormat::YCoCg_DXT5_Alpha_BC4)
	{
		return this;
	}
	// All other versions might need SW fallback - check if we have a real texture as source -> converter needed
	return bHaveTexture ? this : nullptr;
}


#ifdef ELECTRA_HAVE_DX11
struct FRHICommandCopyResourceDX11 final : public FRHICommand<FRHICommandCopyResourceDX11>
{
	TRefCountPtr<ID3D11Texture2D> SampleTexture;
	FTextureRHIRef SampleDestinationTexture;
	bool bCrossDevice;

	FRHICommandCopyResourceDX11(ID3D11Texture2D* InSampleTexture, FRHITexture* InSampleDestinationTexture, bool bInCrossDevice)
		: SampleTexture(InSampleTexture)
		, SampleDestinationTexture(InSampleDestinationTexture)
		, bCrossDevice(bInCrossDevice)
	{
	}

	void Execute(FRHICommandListBase& CmdList)
	{
		LLM_SCOPE(ELLMTag::MediaStreaming);
		ID3D11Device* D3D11Device = GetID3D11DynamicRHI()->RHIGetDevice();
		ID3D11DeviceContext* D3D11DeviceContext = nullptr;

		D3D11Device->GetImmediateContext(&D3D11DeviceContext);
		if (D3D11DeviceContext)
		{
			ID3D11Resource* DestinationTexture = reinterpret_cast<ID3D11Resource*>(SampleDestinationTexture->GetNativeResource());
			if (DestinationTexture)
			{
				if (bCrossDevice)
				{
					TRefCountPtr<IDXGIResource> OtherResource(nullptr);
					SampleTexture->QueryInterface(__uuidof(IDXGIResource), (void**)OtherResource.GetInitReference());

					if (OtherResource)
					{
						//
						// Copy shared texture from decoder device to render device
						//
						HANDLE SharedHandle = nullptr;
						if (OtherResource->GetSharedHandle(&SharedHandle) == S_OK)
						{
							if (SharedHandle != 0)
							{
								TRefCountPtr<ID3D11Resource> SharedResource;
								D3D11Device->OpenSharedResource(SharedHandle, __uuidof(ID3D11Texture2D), (void**)SharedResource.GetInitReference());

								if (SharedResource)
								{
									TRefCountPtr<IDXGIKeyedMutex> KeyedMutex;
									SharedResource->QueryInterface(_uuidof(IDXGIKeyedMutex), (void**)KeyedMutex.GetInitReference());

									if (KeyedMutex)
									{
										// Key is 1 : Texture as just been updated
										// Key is 2 : Texture as already been updated.
										// Do not wait to acquire key 1 since there is race no condition between writer and reader.
										HRESULT Result = KeyedMutex->AcquireSync(1, 0);
										if (Result == S_OK)
										{
											// Copy from shared texture of FSink device to Rendering device
											D3D11DeviceContext->CopyResource(DestinationTexture, SharedResource);
											KeyedMutex->ReleaseSync(2);
										}
										else if (Result == ((HRESULT)WAIT_TIMEOUT))
										{
											// If key 1 cannot be acquired, the resource has already been shutdown or consumed by another reader
											UE_LOGF(LogElectraSamples, Warning, "AcquireSync timed out, DecoderOutput has likely been shut down already!");
										}
										else
										{
											UE_LOGF(LogElectraSamples, Warning, "AcquireSync failed with 0x%08x!", Result);
										}
									}
								}
							}
						}
					}
				}
				else
				{
					//
					// Simple copy on render device
					//
					D3D11DeviceContext->CopyResource(DestinationTexture, SampleTexture);
				}
			}
			D3D11DeviceContext->Release();
		}
	}
};
#endif

/**
 * "Converter" for textures - here: a copy from the decoder owned texture (possibly in another device) into a RHI one (as prep for the real conversion to RGB etc.)
 */
bool FElectraTextureSample::Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints)
{
	LLM_SCOPE(ELLMTag::MediaStreaming);

	RHI_BREADCRUMB_EVENT_STAT(RHICmdList, MediaWinDecoder_Convert, "MediaWinDecoder_Convert");

	// Is this YCoCg data?
	if (SampleFormat == EMediaTextureSampleFormat::YCoCg_DXT5_Alpha_BC4)
	{
		UE_LOGF(LogElectraSamples, Error, "EMediaTextureSampleFormat::YCoCg_DXT5_Alpha_BC4 special conversion code not yet implemented!");
		return false;
	}

	const bool bHaveTexture = TextureDX11 || TextureDX12;
	// If we have a texture and D3D12 is active, we will not need to do any custom conversion work, but we will need a sync!
	if (bHaveTexture && (RHIGetInterfaceType() == ERHIInterfaceType::D3D12))
	{
		uint64 SyncFenceValue;
		TSharedPtr<std::atomic<bool>> bFenceSignalled;
		TRefCountPtr<IUnknown> SyncCommon = GetSync(SyncFenceValue, bFenceSignalled);
		if (SyncCommon)
		{
			//
			// Synchronize with sample data copy on copy queue
			//
			TRefCountPtr<ID3D12Fence> SyncFence;
			HRESULT Result = SyncCommon->QueryInterface(__uuidof(ID3D12Fence), (void**)SyncFence.GetInitReference());
			if (SUCCEEDED(Result))
			{
				RHICmdList.EnqueueLambda([SyncFence, SyncFenceValue, bFenceSignalled](FRHICommandList& RHICmdList)
				{
					GetID3D12DynamicRHI()->RHIWaitManualFence(RHICmdList, SyncFence, SyncFenceValue, bFenceSignalled);
				});
				return true;
			}

			// Only continue with fallback if no interface so that we don't lose the other errors.
			if (Result == E_NOINTERFACE)
			{
				// Support IDXGIKeyedMutex (d3d11 texture path)
				TRefCountPtr<IDXGIKeyedMutex> KeyedMutex;
				Result = SyncCommon->QueryInterface(_uuidof(IDXGIKeyedMutex), (void**)&KeyedMutex);
				if (SUCCEEDED(Result))
				{
					// Remark:
					//	In the d3d11 texture path, the d3d11 context should have already been flushed and
					//	the AcquireSync will not wait in that case. Adding the sync command in any case for extra protection.
					RHICmdList.EnqueueLambda([KeyedMutex](FRHICommandList& RHICmdList)
					{
						if (KeyedMutex)
						{
							// Should we limit the wait as a precaution? ex: 16 ms instead of infinite?
							if (KeyedMutex->AcquireSync(1, INFINITE) == S_OK)
							{
								KeyedMutex->ReleaseSync(2);
							}
						}
					});
					return true;
				}
			}
			return SUCCEEDED(Result);
		}
		return true;
	}

	// Note: the converter code (from here on) is not used at all if we have texture data in a SW buffer!
	check(bHaveTexture);

	bool bCrossDeviceCopy = false;
	EPixelFormat Format = GetPixelFormat();
	if (SourceType != ESourceType::SharedTextureDX11)
	{
		// Below case deliver DX11 textures on the rendering device, so it would be feasible to create an RHI texture from them directly.
		// The current code instead uses a copy of the data to populate an RHI texture. As this is DX11 only and is not the code path
		// affecting main line hardware decoders (H.264/5), we do currently NOT optimize for this.

		if (SourceType != ESourceType::ResourceDX12)
		{
			//
			// SW decoder has decoded into a HW texture (not known to RHI) -> copy it into an RHI one
			//
			check(Format == EPixelFormat::PF_NV12);
			Format = EPixelFormat::PF_G8; // use fixed format: we flag this as NV12, too - as it is - but DX11 will only support a "higher than normal G8 texture" (with specialized access in shader)
			bCrossDeviceCopy = false;
		}
		else
		{
			// We have a texture on the rendering device
			bCrossDeviceCopy = false;
		}
	}
	else
	{
		//
		// HW decoder has delivered a texture (this is already a copy) which is on its own device. Copy into one created by RHI (and hence on our rendering device)
		//
		// note: on DX platforms we won't get any SRV generated for NV12 -> so any user needs to do that manually! (as they please: R8, R8G8...)
		bCrossDeviceCopy = true;
	}

	// Do we need a new RHI texture?
	if (!Texture.IsValid() || Texture->GetSizeX() != SampleDim.X || Texture->GetSizeY() != SampleDim.Y)
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("FElectraTextureSample"), SampleDim, Format)
			.SetFlags((bCanUseSRGB && IsOutputSrgb()) ? ETextureCreateFlags::SRGB : ETextureCreateFlags::None);

		Texture = RHICmdList.CreateTexture(Desc);
	}

	uint64 SyncValue = 0;
	TSharedPtr<std::atomic<bool>> bFenceSignalled;
	TRefCountPtr<IUnknown> TextureCommon = GetTextureD3D();
	TRefCountPtr<IUnknown> SyncCommon = GetSync(SyncValue, bFenceSignalled);

#ifdef ELECTRA_HAVE_DX11
	// DX11 texture?
	TRefCountPtr<ID3D11Texture2D> CommonTextureDX11;
	HRESULT Result = TextureCommon->QueryInterface(__uuidof(ID3D11Texture2D), (void**)CommonTextureDX11.GetInitReference());
	if (SUCCEEDED(Result))
	{
		check(RHIGetInterfaceType() == ERHIInterfaceType::D3D11);

		// Copy data into RHI texture
		if (RHICmdList.Bypass())
		{
			FRHICommandCopyResourceDX11 Cmd(CommonTextureDX11, Texture, bCrossDeviceCopy);
			Cmd.Execute(RHICmdList);
		}
		else
		{
			new (RHICmdList.AllocCommand<FRHICommandCopyResourceDX11>()) FRHICommandCopyResourceDX11(CommonTextureDX11, Texture, bCrossDeviceCopy);
		}
	}
	else
#endif
	{
		// We really only expect to be called for DX11
		check(!"Unexpected call to conversion method from another RHI than the D3D11 one!");
		return false;
	}
	return true;
}


const void* FElectraTextureSample::GetBuffer()
{
	return Buffer ? Buffer->GetData() : nullptr;
}


uint32 FElectraTextureSample::GetStride() const
{
	return Stride;
}

EMediaTextureSampleFormat FElectraTextureSample::GetFormat() const
{
	return SampleFormat;
}


float FElectraTextureSample::GetSampleDataScale(bool b10Bit) const
{
	return IElectraTextureSampleBase::GetSampleDataScale(b10Bit);
}


TRefCountPtr<IUnknown> FElectraTextureSample::GetSync(uint64& OutSyncValue, TSharedPtr<std::atomic<bool>>& OutFenceSignalled)
{
	if (TextureDX11)
	{
		TRefCountPtr<IUnknown> KeyedMutex;
		/*HRESULT Result =*/ TextureDX11->QueryInterface(_uuidof(IDXGIKeyedMutex), (void**)&KeyedMutex);
		OutSyncValue = 0;
		OutFenceSignalled = nullptr;
		return KeyedMutex;
	}
	else if (TextureDX12)
	{
		OutSyncValue = FenceValue;
		OutFenceSignalled = FenceSignalled;
		return static_cast<IUnknown*>(D3DFence.GetReference());
	}
	return nullptr;
}

TRefCountPtr<IUnknown> FElectraTextureSample::GetTextureD3D() const
{
	return TextureDX11 ? static_cast<IUnknown*>(TextureDX11.GetReference()) : static_cast<IUnknown*>(TextureDX12.GetReference());
}


#else // !UE_SERVER

FElectraTextureSample::~FElectraTextureSample()
{ }

void FElectraTextureSample::ClearDX11Vars()
{ }

void FElectraTextureSample::ClearDX12Vars()
{ }

void FElectraTextureSample::ClearBufferVars()
{ }

bool FElectraTextureSample::FinishInitialization()
{ return false; }

#if WITH_ENGINE
FRHITexture* FElectraTextureSample::GetTexture() const
{ return nullptr; }
#endif //WITH_ENGINE


IMediaTextureSampleConverter* FElectraTextureSample::GetMediaTextureSampleConverter()
{ return nullptr; }

bool FElectraTextureSample::Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints)
{ return false; }

const void* FElectraTextureSample::GetBuffer()
{ return nullptr; }


uint32 FElectraTextureSample::GetStride() const
{ return 0; }

EMediaTextureSampleFormat FElectraTextureSample::GetFormat() const
{ return EMediaTextureSampleFormat::Undefined; }

float FElectraTextureSample::GetSampleDataScale(bool b10Bit) const
{ return 1.0f; }

TRefCountPtr<IUnknown> FElectraTextureSample::GetSync(uint64& OutSyncValue, TSharedPtr<std::atomic<bool>>& OutFenceSignalled)
{ OutSyncValue = 0; if(OutFenceSignalled){*OutFenceSignalled=false;} return nullptr; }

TRefCountPtr<IUnknown> FElectraTextureSample::GetTextureD3D() const
{ return nullptr; }


#endif // !UE_SERVER
#endif // PLATFORM_WINDOWS
