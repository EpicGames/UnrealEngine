// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsElectraTextureSampleGPUBufferHelper.h"

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

#if defined(_WIN64) && defined(__SSE2__)

#define ALLOW_USE_OF_SSEMEMCPY 1

#include <intrin.h>

static void SSE2MemCpy(const void* InDst, const void* InSrc, unsigned int InSize)
{
	__m128i* Dst = (__m128i*)InDst;
	__m128i* Src = (__m128i*)InSrc;

	InSize = InSize >> 7;
	for(unsigned int i = 0; i < InSize; i++)
	{
		_mm_prefetch((char*)(Src + 8), 0);
		_mm_prefetch((char*)(Src + 10), 0);
		_mm_prefetch((char*)(Src + 12), 0);
		_mm_prefetch((char*)(Src + 14), 0);
		__m128i m0 = _mm_load_si128(Src + 0);
		__m128i m1 = _mm_load_si128(Src + 1);
		__m128i m2 = _mm_load_si128(Src + 2);
		__m128i m3 = _mm_load_si128(Src + 3);
		__m128i m4 = _mm_load_si128(Src + 4);
		__m128i m5 = _mm_load_si128(Src + 5);
		__m128i m6 = _mm_load_si128(Src + 6);
		__m128i m7 = _mm_load_si128(Src + 7);
		_mm_stream_si128(Dst + 0, m0);
		_mm_stream_si128(Dst + 1, m1);
		_mm_stream_si128(Dst + 2, m2);
		_mm_stream_si128(Dst + 3, m3);
		_mm_stream_si128(Dst + 4, m4);
		_mm_stream_si128(Dst + 5, m5);
		_mm_stream_si128(Dst + 6, m6);
		_mm_stream_si128(Dst + 7, m7);
		Src += 8;
		Dst += 8;
	}
}

#else
#define ALLOW_USE_OF_SSEMEMCPY 0
#endif // _WIN64

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

struct FElectraMediaDecoderOutputBufferPoolBlock_DX12::FResourceInfo
{
	FResourceInfo(TWeakPtr<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe> InPool, uint32 InBlockIdx, uint32 InBufferIdx, TRefCountPtr<ID3D12Heap> InHeap) : Pool(InPool), BlockIdx(InBlockIdx), BufferIdx(InBufferIdx), Heap(InHeap) {}

	TWeakPtr<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe> Pool;
	uint32 BlockIdx;
	uint32 BufferIdx;
	TRefCountPtr<ID3D12Heap> Heap;
};


// Create instance for use with buffers (usually for uploading data)
FElectraMediaDecoderOutputBufferPoolBlock_DX12::FElectraMediaDecoderOutputBufferPoolBlock_DX12(uint32 InBlockIdx, TRefCountPtr<ID3D12Device> InD3D12Device, uint32 InMaxNumBuffers, uint32 Width, uint32 Height, uint32 BytesPerPixel, D3D12_HEAP_TYPE InD3D12HeapType)
	: D3D12Device(InD3D12Device)
	, D3D12HeapType(InD3D12HeapType)
	, MaxNumBuffers(InMaxNumBuffers)
	, FreeMask((uint32)((1ULL << InMaxNumBuffers) - 1))
	, BlockIdx(InBlockIdx)
{
	check(InMaxNumBuffers <= 32);

	// Get the size needed per buffer
	BufferPitch = Align(Width * BytesPerPixel, D3D12_TEXTURE_DATA_PITCH_ALIGNMENT);
	BufferSize = Align(BufferPitch * Height, FMath::Max(D3D12_TEXTURE_DATA_PLACEMENT_ALIGNMENT, D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT));

	InitCommon();
}

// Create instance for use with textures (usually to receive any upload buffers or internal decoder texture data)
FElectraMediaDecoderOutputBufferPoolBlock_DX12::FElectraMediaDecoderOutputBufferPoolBlock_DX12(uint32 InBlockIdx, TRefCountPtr<ID3D12Device> InD3D12Device, uint32 InMaxNumBuffers, uint32 Width, uint32 Height, DXGI_FORMAT PixFmt, D3D12_HEAP_TYPE InD3D12HeapType)
	: D3D12Device(InD3D12Device)
	, D3D12HeapType(InD3D12HeapType)
	, MaxNumBuffers(InMaxNumBuffers)
	, FreeMask((uint32)((1ULL << InMaxNumBuffers) - 1))
	, BlockIdx(InBlockIdx)
{
	check(InMaxNumBuffers <= 32);

	// Get the size needed per buffer / texture
	D3D12_RESOURCE_DESC Desc = {};
	Desc.MipLevels = 1;
	Desc.Format = PixFmt;
	Desc.Width = Width;
	Desc.Height = Height;
	Desc.Flags = D3D12_RESOURCE_FLAG_NONE;
	Desc.DepthOrArraySize = 1;
	Desc.SampleDesc.Count = 1;
	Desc.SampleDesc.Quality = 0;
	Desc.Layout = D3D12_TEXTURE_LAYOUT_UNKNOWN;
	Desc.Dimension = D3D12_RESOURCE_DIMENSION_TEXTURE2D;
	D3D12_RESOURCE_ALLOCATION_INFO AllocInfo = InD3D12Device->GetResourceAllocationInfo(0, 1, &Desc);

	BufferPitch = 0;
	BufferSize = Align(AllocInfo.SizeInBytes, AllocInfo.Alignment);
	InitCommon();
}

FElectraMediaDecoderOutputBufferPoolBlock_DX12::~FElectraMediaDecoderOutputBufferPoolBlock_DX12()
{
}

uint32 FElectraMediaDecoderOutputBufferPoolBlock_DX12::GetMaxNumBuffers() const
{
	return MaxNumBuffers;
}

// Check if the current setup is compatible with the new parameters
bool FElectraMediaDecoderOutputBufferPoolBlock_DX12::IsCompatibleAsBuffer(uint32 Width, uint32 Height, uint32 BytesPerPixel) const
{
	uint32 NewBufferPitch = Align(Width * BytesPerPixel, D3D12_TEXTURE_DATA_PITCH_ALIGNMENT);
	uint64 NewBufferSize = Align(NewBufferPitch * Height, FMath::Max(D3D12_TEXTURE_DATA_PLACEMENT_ALIGNMENT, D3D12_DEFAULT_RESOURCE_PLACEMENT_ALIGNMENT));
	return BufferSize >= NewBufferSize && BufferPitch == NewBufferPitch;
}

// Check if the current setup is compatible with the new parameters
bool FElectraMediaDecoderOutputBufferPoolBlock_DX12::IsCompatibleAsTexture(uint32 Width, uint32 Height, DXGI_FORMAT PixFmt) const
{
	// Get the size needed per buffer / texture
	D3D12_RESOURCE_DESC Desc = {};
	Desc.MipLevels = 1;
	Desc.Format = PixFmt;
	Desc.Width = Width;
	Desc.Height = Height;
	Desc.Flags = D3D12_RESOURCE_FLAG_NONE;
	Desc.DepthOrArraySize = 1;
	Desc.SampleDesc.Count = 1;
	Desc.SampleDesc.Quality = 0;
	Desc.Layout = D3D12_TEXTURE_LAYOUT_UNKNOWN;
	Desc.Dimension = D3D12_RESOURCE_DIMENSION_TEXTURE2D;
	D3D12_RESOURCE_ALLOCATION_INFO AllocInfo = D3D12Device->GetResourceAllocationInfo(0, 1, &Desc);

	uint64 NewBufferSize = Align(AllocInfo.SizeInBytes, AllocInfo.Alignment);
	return BufferSize >= NewBufferSize;
}

// Check if a buffer is available
bool FElectraMediaDecoderOutputBufferPoolBlock_DX12::BufferAvailable() const
{
	return FPlatformAtomics::AtomicRead((const int32*)&FreeMask) != 0;
}

// Allocate a buffer resource and return suitable sync fence data
TRefCountPtr<ID3D12Resource> FElectraMediaDecoderOutputBufferPoolBlock_DX12::AllocateOutputDataAsBuffer(HRESULT& OutResult, FString& OutMessage, TWeakPtr<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe> InPoolWeakRef, uint32& OutPitch)
{
	TRefCountPtr<ID3D12Resource> Resource = AllocateBuffer(OutResult, OutMessage, InPoolWeakRef, OutPitch, [NumBytes = BufferSize](D3D12_RESOURCE_DESC& Desc)
	{
		Desc.MipLevels = 1;
		Desc.Format = DXGI_FORMAT_UNKNOWN;
		Desc.Width = NumBytes;
		Desc.Height = 1;
		Desc.Flags = D3D12_RESOURCE_FLAG_NONE;
		Desc.DepthOrArraySize = 1;
		Desc.SampleDesc.Count = 1;
		Desc.SampleDesc.Quality = 0;
		Desc.Layout = D3D12_TEXTURE_LAYOUT_ROW_MAJOR;
		Desc.Dimension = D3D12_RESOURCE_DIMENSION_BUFFER;
	});
	return Resource;
}

// Allocate a texture resource and return suitable sync fence data
TRefCountPtr<ID3D12Resource> FElectraMediaDecoderOutputBufferPoolBlock_DX12::AllocateOutputDataAsTexture(HRESULT& OutResult, FString& OutMessage, TWeakPtr<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe> InPoolWeakRef, uint32 InWidth, uint32 InHeight, DXGI_FORMAT InPixFmt)
{
	uint32 OutPitch;
	TRefCountPtr<ID3D12Resource> Resource = AllocateBuffer(OutResult, OutMessage, InPoolWeakRef, OutPitch, [InWidth, InHeight, InPixFmt](D3D12_RESOURCE_DESC& Desc)
	{
		Desc.MipLevels = 1;
		Desc.Format = InPixFmt;
		Desc.Width = InWidth;
		Desc.Height = InHeight;
		Desc.Flags = D3D12_RESOURCE_FLAG_NONE;
		Desc.DepthOrArraySize = 1;
		Desc.SampleDesc.Count = 1;
		Desc.SampleDesc.Quality = 0;
		Desc.Layout = D3D12_TEXTURE_LAYOUT_UNKNOWN;
		Desc.Dimension = D3D12_RESOURCE_DIMENSION_TEXTURE2D;
	});
	return Resource;
}

void FElectraMediaDecoderOutputBufferPoolBlock_DX12::InitCommon()
{
	D3D12_HEAP_DESC HeapDesc = {};
	HeapDesc.Alignment = 0;
	HeapDesc.SizeInBytes = BufferSize * MaxNumBuffers;
	check(HeapDesc.SizeInBytes <= 0xffff0000U);
	HeapDesc.Properties = CD3DX12_HEAP_PROPERTIES(D3D12HeapType);
	HeapDesc.Flags = D3D12_HEAP_FLAG_CREATE_NOT_ZEROED | ((D3D12HeapType == D3D12_HEAP_TYPE_UPLOAD) ? D3D12_HEAP_FLAG_ALLOW_ONLY_BUFFERS : D3D12_HEAP_FLAG_ALLOW_ONLY_NON_RT_DS_TEXTURES);
	HRESULT Res = D3D12Device->CreateHeap(&HeapDesc, IID_PPV_ARGS(D3D12OutputHeap.GetInitReference()));
	if (Res != S_OK)
	{
		FreeMask = 0;
	}
#if !UE_BUILD_SHIPPING
	if (D3D12OutputHeap)
	{
		D3D12OutputHeap->SetName(TEXT("ElectraOutputBufferPoolHeap"));
	}
#endif
}

TRefCountPtr<ID3D12Resource> FElectraMediaDecoderOutputBufferPoolBlock_DX12::AllocateBuffer(HRESULT& OutResult, FString& OutMessage, TWeakPtr<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe> InPoolWeakRef, uint32& OutBufferPitch, TFunction<void(D3D12_RESOURCE_DESC& Desc)> && InitializeDesc)
{
	TRefCountPtr<ID3D12Resource> Resource;
	OutResult = S_OK;

	// Find the buffer index we can use...
	int32 BufferIdx, OldFreeMask, NewFreeMask;
	do
	{
		OldFreeMask = (int32)FreeMask;
		if (OldFreeMask == 0)
		{
			BufferIdx = -1;
			break;
		}
		BufferIdx = 31 - FMath::CountLeadingZeros(OldFreeMask);
		NewFreeMask = OldFreeMask & ~(1 << BufferIdx);
	} while (OldFreeMask != FPlatformAtomics::InterlockedCompareExchange((int32*)&FreeMask, NewFreeMask, OldFreeMask));

	// Anything?
	if (BufferIdx >= 0)
	{
		// Make a placed resource to use...
		D3D12_RESOURCE_DESC Desc = {};
		InitializeDesc(Desc);
		OutResult = D3D12Device->CreatePlacedResource(D3D12OutputHeap, BufferSize * BufferIdx, &Desc, (D3D12HeapType == D3D12_HEAP_TYPE_UPLOAD) ? D3D12_RESOURCE_STATE_GENERIC_READ : D3D12_RESOURCE_STATE_COMMON, nullptr, IID_PPV_ARGS(Resource.GetInitReference()));
		if (OutResult == S_OK)
		{
#if !UE_BUILD_SHIPPING
			Resource->SetName(TEXT("ElectraOutputBufferPoolBufferResource"));
#endif

			OutBufferPitch = BufferPitch;

			// Register a destruction callback so we can reset our memory management here transparently...
			TRefCountPtr<ID3DDestructionNotifier> Notifier;
			OutResult = Resource->QueryInterface(__uuidof(ID3DDestructionNotifier), (void**)Notifier.GetInitReference());
			if (SUCCEEDED(OutResult))
			{
				// note: we keep a reference to the heap in our context data for the destruction callback, so we can ensure the heap lives as long as there are placed resources in it
				UINT CallbackID;
				TUniquePtr<FResourceInfo> ResourceInfo(new FResourceInfo(InPoolWeakRef, BlockIdx, BufferIdx, D3D12OutputHeap));
				OutResult = Notifier->RegisterDestructionCallback(ResourceDestructionCallback, ResourceInfo.Get(), &CallbackID);
				if ((SUCCEEDED(OutResult)))
				{
					(void)ResourceInfo.Release();
				}
				else
				{
					Resource = nullptr;
					FreeBuffer(BufferIdx);
					OutMessage = TEXT("Call to RegisterDestructionCallback() failed");
				}
			}
			else
			{
				Resource = nullptr;
				FreeBuffer(BufferIdx);
				OutMessage = TEXT("Call to QueryInterface(ID3DDestructionNotifier) failed");
			}
		}
		else
		{
			OutMessage = TEXT("Call to CreatePlacedResource() failed");
			FreeBuffer(BufferIdx);
		}
	}

	return Resource;
}

void FElectraMediaDecoderOutputBufferPoolBlock_DX12::FreeBuffer(int32 InBufferIdx)
{
	// Reset buffer index to free...
	int32 OldFreeMask, NewFreeMask;
	do
	{
		OldFreeMask = (int32)FreeMask;
		NewFreeMask = OldFreeMask | (1 << InBufferIdx);
	} while (OldFreeMask != FPlatformAtomics::InterlockedCompareExchange((int32*)&FreeMask, NewFreeMask, OldFreeMask));
}


// Create instance for use with buffers (usually for uploading data)
FElectraMediaDecoderOutputBufferPool_DX12::FElectraMediaDecoderOutputBufferPool_DX12(TRefCountPtr<ID3D12Device> InD3D12Device, uint32 InMaxNumBuffers, uint32 InWidth, uint32 InHeight, uint32 InBytesPerPixel, D3D12_HEAP_TYPE InD3D12HeapType)
	: D3D12Device(InD3D12Device)
	, LastFenceValue(0)
	, D3D12HeapType(InD3D12HeapType)
	, Width(InWidth)
	, Height(InHeight)
	, PixFmt(DXGI_FORMAT_UNKNOWN)
	, BytesPerPixel(InBytesPerPixel)
	, MaxNumBuffers(InMaxNumBuffers)
{
	MaxBufsPerBlock = MaxNumBuffers < kMaxBufsPerBlock ? MaxNumBuffers : kMaxBufsPerBlock;
	Blocks.Emplace(new FElectraMediaDecoderOutputBufferPoolBlock_DX12(0, D3D12Device, MaxBufsPerBlock, Width, Height, BytesPerPixel, D3D12HeapType));
	InitCommon();
}

// Create instance for use with textures (usually to receive any upload buffers or internal decoder texture data)
FElectraMediaDecoderOutputBufferPool_DX12::FElectraMediaDecoderOutputBufferPool_DX12(TRefCountPtr<ID3D12Device> InD3D12Device, uint32 InMaxNumBuffers, uint32 InWidth, uint32 InHeight, DXGI_FORMAT InPixFmt, D3D12_HEAP_TYPE InD3D12HeapType)
	: D3D12Device(InD3D12Device)
	, LastFenceValue(0)
	, D3D12HeapType(InD3D12HeapType)
	, Width(InWidth)
	, Height(InHeight)
	, PixFmt(InPixFmt)
	, BytesPerPixel(0)
	, MaxNumBuffers(InMaxNumBuffers)
{
	MaxBufsPerBlock = MaxNumBuffers < kMaxBufsPerBlock ? MaxNumBuffers : kMaxBufsPerBlock;
	Blocks.Emplace(new FElectraMediaDecoderOutputBufferPoolBlock_DX12(0, D3D12Device, MaxBufsPerBlock, Width, Height, PixFmt, D3D12HeapType));
	InitCommon();
}

FElectraMediaDecoderOutputBufferPool_DX12::~FElectraMediaDecoderOutputBufferPool_DX12()
{
	for(FElectraMediaDecoderOutputBufferPoolBlock_DX12* Block : Blocks)
	{
		delete Block;
	}
}

// Check if the current setup is compatible with the new parameters
bool FElectraMediaDecoderOutputBufferPool_DX12::IsCompatibleAsBuffer(uint32 InMaxNumBuffers, uint32 InWidth, uint32 InHeight, uint32 InBytesPerPixel) const
{
	return InMaxNumBuffers <= MaxNumBuffers ? Blocks[0]->IsCompatibleAsBuffer(InWidth, InHeight, InBytesPerPixel) : false;
}

bool FElectraMediaDecoderOutputBufferPool_DX12::IsCompatibleAsTexture(uint32 InMaxNumBuffers, uint32 InWidth, uint32 InHeight, DXGI_FORMAT InPixFmt) const
{
	return PixFmt == InPixFmt && InMaxNumBuffers <= MaxNumBuffers ? Blocks[0]->IsCompatibleAsTexture(InWidth, InHeight, InPixFmt) : false;
}

// Check if a buffer is available
bool FElectraMediaDecoderOutputBufferPool_DX12::BufferAvailable() const
{
	// If there is something in the pending free queue then we have room in some block.
	if (!PendingFreeQueue.IsEmpty())
	{
		return true;
	}
	// Check if any already existing block has an available entry
	uint32 NumAllocatedBuffers = 0;
	for(FElectraMediaDecoderOutputBufferPoolBlock_DX12* Block : Blocks)
	{
		NumAllocatedBuffers += Block->MaxNumBuffers;
		if (Block->BufferAvailable())
		{
			return true;
		}
	}
	// Still possible to add new buffers?
	return NumAllocatedBuffers < MaxNumBuffers;
}

bool FElectraMediaDecoderOutputBufferPool_DX12::AllocateOutputDataAsBuffer(HRESULT& OutResult, FString& OutMessage, FOutputData& OutData, uint32& OutPitch)
{
	check(BytesPerPixel != 0);

	OutResult = S_OK;
	OutData.Resource = nullptr;
	HandlePendingFrees();
	for(FElectraMediaDecoderOutputBufferPoolBlock_DX12* Block : Blocks)
	{
		if (Block->BufferAvailable())
		{
			OutData.Resource = Block->AllocateOutputDataAsBuffer(OutResult, OutMessage, AsWeak(), OutPitch);
			if (OutResult != S_OK || OutData.Resource)
			{
				break;
			}
		}
	}

	// Allocate a new block if we could not get a buffer, but only when there is no error yet.
	if (!OutData.Resource.IsValid() && OutResult == S_OK)
	{
		const uint32 AdditionalBuffers = kMaxBufsPerBlock;
		Blocks.Emplace(new FElectraMediaDecoderOutputBufferPoolBlock_DX12(Blocks.Num(), D3D12Device, AdditionalBuffers, Width, Height, BytesPerPixel, D3D12HeapType));
		OutData.Resource = Blocks.Last()->AllocateOutputDataAsBuffer(OutResult, OutMessage, AsWeak(), OutPitch);
	}

	if (OutData.Resource.IsValid())
	{
		OutData.Fence = GetUpdatedBufferFence(OutData.FenceValue);
		return true;
	}
	return false;
}

bool FElectraMediaDecoderOutputBufferPool_DX12::AllocateOutputDataAsTexture(HRESULT& OutResult, FString& OutMessage, FOutputData& OutData, uint32 InWidth, uint32 InHeight, DXGI_FORMAT InPixFmt)
{
	check(PixFmt != DXGI_FORMAT_UNKNOWN);

	OutData.Resource = nullptr;
	OutResult = S_OK;
	HandlePendingFrees();
	for(FElectraMediaDecoderOutputBufferPoolBlock_DX12* Block : Blocks)
	{
		if (Block->BufferAvailable())
		{
			OutData.Resource = Block->AllocateOutputDataAsTexture(OutResult, OutMessage, AsWeak(), InWidth, InHeight, InPixFmt);
			if (OutResult != S_OK || OutData.Resource)
			{
				break;
			}
		}
	}

	// Allocate a new block if we could not get a buffer, but only when there is no error yet.
	if (!OutData.Resource.IsValid() && OutResult == S_OK)
	{
		const uint32 AdditionalBuffers = kMaxBufsPerBlock;
		Blocks.Emplace(new FElectraMediaDecoderOutputBufferPoolBlock_DX12(Blocks.Num(), D3D12Device, AdditionalBuffers, Width, Height, PixFmt, D3D12HeapType));
		OutData.Resource = Blocks.Last()->AllocateOutputDataAsTexture(OutResult, OutMessage, AsWeak(), InWidth, InHeight, InPixFmt);
	}

	if (OutData.Resource.IsValid())
	{
		OutData.Fence = GetUpdatedBufferFence(OutData.FenceValue);
		return true;
	}
	return false;
}

TRefCountPtr<ID3D12Fence> FElectraMediaDecoderOutputBufferPool_DX12::GetUpdatedBufferFence(uint64& FenceValue)
{
	FenceValue = ++LastFenceValue;
	return D3D12BufferFence;
}

// Helper function to copy simple, linear texture data between differently picthed buffers
void FElectraMediaDecoderOutputBufferPool_DX12::CopyWithPitchAdjust(uint8* Dst, uint32 DstPitch, const uint8* Src, uint32 SrcPitch, uint32 NumRows)
{
	if (DstPitch != SrcPitch)
	{
		for(uint32 Y = NumRows; Y > 0; --Y)
		{
			FMemory::Memcpy(Dst, Src, SrcPitch);
			Src += SrcPitch;
			Dst += DstPitch;
		}
	}
	else
	{
		uint32 NumBytes = SrcPitch * NumRows;
#if ALLOW_USE_OF_SSEMEMCPY
		if ((NumBytes & 0x7f) == 0)
		{
			SSE2MemCpy(Dst, Src, NumBytes);
		}
		else
#endif
		{
			FMemory::Memcpy(Dst, Src, NumBytes);
		}
	}
}


void FElectraMediaDecoderOutputBufferPool_DX12::InitCommon()
{
	HRESULT Res = D3D12Device->CreateFence(0, D3D12_FENCE_FLAG_NONE, IID_PPV_ARGS(D3D12BufferFence.GetInitReference()));
	if (SUCCEEDED(Res))
	{
#if !UE_BUILD_SHIPPING
		D3D12BufferFence->SetName(TEXT("ElectraOutputBufferPoolFence"));
#endif
	}
}

void FElectraMediaDecoderOutputBufferPool_DX12::AddPendingFree(uint32 InBlockIndex, uint32 InBufferIndex)
{
	FPendingFreeEntry pe { InBlockIndex, InBufferIndex };
	PendingFreeQueue.Enqueue(MoveTemp(pe));
}

void FElectraMediaDecoderOutputBufferPool_DX12::HandlePendingFrees()
{
	FPendingFreeEntry pe;
	while(PendingFreeQueue.Dequeue(pe))
	{
		check((uint32)Blocks.Num() > pe.BlockIndex);
		Blocks[pe.BlockIndex]->FreeBuffer(pe.BufferIndex);
	}
}

void FElectraMediaDecoderOutputBufferPoolBlock_DX12::ResourceDestructionCallback(void* Context)
{
	auto ResourceInfo = reinterpret_cast<FResourceInfo*>(Context);

	if (auto Pool = ResourceInfo->Pool.Pin())
	{
		Pool->AddPendingFree(ResourceInfo->BlockIdx, ResourceInfo->BufferIdx);
		//Pool->Blocks[ResourceInfo->BlockIdx]->FreeBuffer(ResourceInfo->BufferIdx);
	}

	// Get rid of our resource tracking info block...
	// (this will also release the ref to the heap the resource was on)
	delete ResourceInfo;
}
