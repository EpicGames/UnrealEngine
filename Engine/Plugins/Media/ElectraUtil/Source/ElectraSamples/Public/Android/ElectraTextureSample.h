// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#include "CoreTypes.h"
#include "CoreGlobals.h"
#include "Templates/SharedPointer.h"
#include "IElectraTextureSample.h"
#include "Math/IntPoint.h"
#include "Math/Range.h"
#include "Misc/Timespan.h"
#include "MediaObjectPool.h"
#include "MediaSamples.h"

#define UE_API ELECTRASAMPLES_API

class FElectraTextureSampleSupport;
class FEvent;
struct AHardwareBuffer;

/**
 * Texture sample generated by AndroidMedia player.
 */
class FElectraTextureSample final
	: public IElectraTextureSampleBase
	, public IMediaTextureSampleConverter
	, public TSharedFromThis<FElectraTextureSample, ESPMode::ThreadSafe>
{
public:
	UE_API FElectraTextureSample(TSharedPtr<FElectraTextureSampleSupport, ESPMode::ThreadSafe> InSupport);
	UE_API virtual ~FElectraTextureSample();

	UE_API bool Initialize();

	UE_API bool FinishInitialization() override;

	//~ IMediaTextureSample interface
	UE_API const void* GetBuffer() override;
	UE_API uint32 GetStride() const override;
	UE_API EMediaTextureSampleFormat GetFormat() const override;
	UE_API FRHITexture* GetTexture() const override;
	UE_API IMediaTextureSampleConverter* GetMediaTextureSampleConverter() override;
	UE_API FIntPoint GetDim() const override;

	UE_API bool IsReadyForReuse() override;
	UE_API void ShutdownPoolable() override;

private:
	UE_API uint32 GetConverterInfoFlags() const override;
	UE_API bool Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints) override;

	friend class FElectraTextureSampleSupport;

	UE_API FTextureRHIRef InitializeTextureOES(AHardwareBuffer* HardwareBuffer);
	UE_API FTextureRHIRef InitializeTextureVulkan(AHardwareBuffer* HardwareBuffer);

	UE_API void InitializeTexture(FRHICommandListBase& RHICmdList, EPixelFormat PixelFormat);
	UE_API void SetupFromBuffer(const void* InBuffer, int32 InBufferSize);
	UE_API void SetImageResources(jobject InImageResources);
	UE_API void CleanupImageResources();
	UE_API void CopyFromExternalTextureOES(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, FTextureRHIRef& InSrcTexture, const FVector2f& Scale, const FVector2f& Offset);
	UE_API void CopyFromExternalTextureVulkan(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, FTextureRHIRef& InSrcTexture, const FVector2f& InScale, const FVector2f& InOffset);

	UE_API bool ConvertCpuOutputPath(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints);
	UE_API bool ConvertGpuOutputPath(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints);


	TSharedPtr<FElectraTextureSampleSupport, ESPMode::ThreadSafe> Support;

	TRefCountPtr<FRHITexture> Texture;
	TRefCountPtr<FRHIGPUFence> Fence;

	void* Buffer = nullptr;
	int32 BufferSize = 0;

	jobject ImageResources = nullptr;
	bool bQueuedForConversion = false;

	bool UseGpuOutputPath = false;
};


using FElectraTextureSamplePtr = TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe>;
using FElectraTextureSampleRef = TSharedRef<FElectraTextureSample, ESPMode::ThreadSafe>;

class FElectraTextureSamplePool : public TMediaObjectPool<FElectraTextureSample, FElectraTextureSamplePool>
{
	using TextureSample = FElectraTextureSample;

public:
	UE_API FElectraTextureSamplePool();

	TextureSample* Alloc() const
	{
		return new TextureSample(Support);
	}

	UE_API void* GetCodecSurface();

private:
	TSharedPtr<FElectraTextureSampleSupport, ESPMode::ThreadSafe> Support;
};

#undef UE_API
