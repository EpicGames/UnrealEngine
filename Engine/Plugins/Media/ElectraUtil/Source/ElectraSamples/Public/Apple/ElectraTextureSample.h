// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#include "MediaSamples.h"
#include "IElectraTextureSample.h"
#include "IMediaTextureSampleConverter.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#import <AVFoundation/AVFoundation.h>
#import <VideoToolbox/VideoToolbox.h>

class FElectraMediaTexConvApple;


class ELECTRASAMPLES_API FElectraTextureSample final
	: public IElectraTextureSampleBase
    , public IMediaTextureSampleConverter
{
public:
	FElectraTextureSample(const TWeakPtr<FElectraMediaTexConvApple, ESPMode::ThreadSafe>& InTexConv);
	virtual ~FElectraTextureSample();

	bool FinishInitialization() override;

	//~ IMediaTextureSample interface
	const void* GetBuffer() override;
	EMediaTextureSampleFormat GetFormat() const override;
	uint32 GetStride() const override;
	FRHITexture* GetTexture() const override;
	IMediaTextureSampleConverter* GetMediaTextureSampleConverter() override;

	void ShutdownPoolable() override;

	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> Buffer;
	CVImageBufferRef ImageBufferRef = nullptr;
	EMediaTextureSampleFormat SampleFormat = EMediaTextureSampleFormat::Undefined;
	uint32 Stride = 0;

	TWeakPtr<FElectraMediaTexConvApple, ESPMode::ThreadSafe> TexConv;

	uint32 GetConverterInfoFlags() const override;
    bool Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints) override;
};


using FElectraTextureSamplePtr = TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe>;
using FElectraTextureSampleRef = TSharedRef<FElectraTextureSample, ESPMode::ThreadSafe>;


class ELECTRASAMPLES_API FElectraMediaTexConvApple
{
public:
    FElectraMediaTexConvApple();
    ~FElectraMediaTexConvApple();

#if WITH_ENGINE
	void ConvertTexture(FTextureRHIRef& InDstTexture, CVImageBufferRef InImageBufferRef, bool bFullRange, EMediaTextureSampleFormat Format, const FMatrix44f& YUVMtx, const UE::Color::FColorSpace& SourceColorSpace, UE::Color::EEncoding EncodingType, float NormalizationFactor);

	UE_DEPRECATED(5.5, "This ConvertTexture function is deprecated, please use the version with FColorSpace instead.")
	void ConvertTexture(FTextureRHIRef& InDstTexture, CVImageBufferRef InImageBufferRef, bool bFullRange, EMediaTextureSampleFormat Format, const FMatrix44f& YUVMtx, const FMatrix44d& GamutToXYZMtx, UE::Color::EEncoding EncodingType, float NormalizationFactor) { }
#endif

private:
#if WITH_ENGINE
    /** The Metal texture cache for unbuffered texture uploads. */
    CVMetalTextureCacheRef MetalTextureCache;
#endif
};


class ELECTRASAMPLES_API FElectraTextureSamplePool : public TMediaObjectPool<FElectraTextureSample, FElectraTextureSamplePool>
{
	using TextureSample = FElectraTextureSample;
public:
	FElectraTextureSamplePool()
		: TMediaObjectPool<TextureSample, FElectraTextureSamplePool>(this)
		, TexConv(new FElectraMediaTexConvApple())
	{}

	TextureSample *Alloc() const
	{
		return new TextureSample(TexConv);
	}

private:
	TSharedPtr<FElectraMediaTexConvApple, ESPMode::ThreadSafe> TexConv;
};
