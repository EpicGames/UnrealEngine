// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// Provide fallback defaults for various defines
#ifndef ELECTRA_MEDIAGPUBUFFER_DX12
#define	ELECTRA_MEDIAGPUBUFFER_DX12 0
#endif

// Decoder settings define the amount of output samples in flight, but this setting omits a few internal instances that
// will occur at the same time. Usually just one, but at times 2, this informs this constant used to make sure pre-allocated
// output buffer heaps are large enough at all times
static const uint32 kElectraDecoderPipelineExtraFrames = 2;
