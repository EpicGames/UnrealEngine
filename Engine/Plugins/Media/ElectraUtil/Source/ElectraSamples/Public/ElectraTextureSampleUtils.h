// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "ColorManagement/ColorManagementDefines.h"
#include "Misc/AssertionMacros.h"

namespace ElectraColorimetryUtils
{
	static constexpr uint8 DefaultMPEGColorPrimaries = 2;
	static constexpr uint8 DefaultMPEGMatrixCoefficients = 2;
	static constexpr uint8 DefaultMPEGTransferCharacteristics = 2;

	inline UE::Color::EColorSpace TranslateMPEGColorPrimaries(uint8 InPrimaries)
	{
		switch (InPrimaries)
		{
			case 1:				// Rec709
			case 2:				// unknown
			{
				return UE::Color::EColorSpace::sRGB;
			}
			case 6:				// Rec601
			case 7:				// Rec601 (historical)
			{
				return UE::Color::EColorSpace::sRGB;
			}
			case 9:				// Rec2020
			{
				return UE::Color::EColorSpace::Rec2020;
			}
			default:
			{
				break;
			}
		}
		return UE::Color::EColorSpace::sRGB;
	}

	inline UE::Color::EColorSpace TranslateMPEGMatrixCoefficients(uint8 InMatrixCoefficients)
	{
		switch (InMatrixCoefficients)
		{
			case 0:				// ID (RGB)
			{
				return UE::Color::EColorSpace::None;
			}
			case 1:				// Rec709
			case 2:				// unknown
			{
				return UE::Color::EColorSpace::sRGB;
			}
			case 6:				// Rec601
			case 7:				// Rec601 (historical)
			{
				return UE::Color::EColorSpace::sRGB;
			}
			case 9:				// Rec2020
			{
				return UE::Color::EColorSpace::Rec2020;
			}
			default:
			{
				break;
			}
		}
		return UE::Color::EColorSpace::sRGB;
	}

	inline UE::Color::EEncoding TranslateMPEGTransferCharacteristics(uint8 InTransferCharacteristics)
	{
		switch (InTransferCharacteristics)
		{
			case 1:
			case 6:
			case 7:
			case 14:
			case 15: // Rec. ITU-R BT.1886-0
			{
				return UE::Color::EEncoding::sRGB;	// using sRGB in place of HLG for now
			}
			case 18: // Rec. ITU-R BT.2100-2 (HLG)
			{
				return UE::Color::EEncoding::HLG;
			}
			case 2:				// unknown
			{
				return UE::Color::EEncoding::sRGB;
			}
			case 8:
			{
				return UE::Color::EEncoding::Linear;
			}
			case 16:
			{
				return UE::Color::EEncoding::ST2084;
			}
			default:
			{
				break;
			}
		}
		return UE::Color::EEncoding::sRGB;
	}
};
