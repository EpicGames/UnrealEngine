// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IMediaAudioSample.h"
#include "Templates/SharedPointer.h"
#include "MediaObjectPool.h"

#define UE_API ELECTRASAMPLES_API


class FElectraAudioSample
	: public IMediaAudioSample
	, public IMediaPoolable
{
public:
	UE_API virtual ~FElectraAudioSample();
	// Prepare the buffer to receive as many samples as specified for the given format, after which the data can be copied into the buffer.
	UE_API bool AllocateFor(EMediaAudioSampleFormat InFormat, uint32 InNumChannels, uint32 InNumFrames);
	uint32 GetAllocatedSize() const
	{ return NumBytesAllocated;}
	void* GetWritableBuffer()
	{ return Buffer; }

	// Sets sample information of the data in the buffer.
	UE_API void SetParameters(uint32 InSampleRate, const FMediaTimeStamp& InTime, const FTimespan& InDuration);
	void SetNumFrames(uint32 InNumFrames)
	{ NumFrames = InNumFrames; }

	// From IMediaAudioSample
	const void* GetBuffer() override
	{ return Buffer; }
	uint32 GetChannels() const override
	{ return NumChannels; }
	FTimespan GetDuration() const override
	{ return Duration; }
	EMediaAudioSampleFormat GetFormat() const override
	{ return MediaAudioSampleFormat; }
	uint32 GetFrames() const override
	{ return NumFrames; }
	uint32 GetSampleRate() const override
	{ return SampleRate; }
	FMediaTimeStamp GetTime() const override
	{ return MediaTimeStamp; }
	//TOptional<FTimecode> GetTimecode() const override { return TOptional<FTimecode>(); }


	DECLARE_DELEGATE_OneParam(FReleaseDelegate, FElectraAudioSample*);
	FReleaseDelegate& GetReleaseDelegate()
	{ return ReleaseDelegate; }

	// From IMediaPoolable
#if !UE_SERVER
	UE_API void ShutdownPoolable() override;
#endif

protected:
	FMediaTimeStamp MediaTimeStamp;
	FTimespan Duration;
	EMediaAudioSampleFormat MediaAudioSampleFormat = EMediaAudioSampleFormat::Float;
	void* Buffer = nullptr;
	uint32 NumChannels = 0;
	uint32 NumFrames = 0;
	uint32 SampleRate = 0;
	uint32 NumBytesAllocated = 0;
	/** Optional delegate to call during ShutdownPoolable(). */
	FReleaseDelegate ReleaseDelegate;
};


using FElectraAudioSamplePtr = TSharedPtr<FElectraAudioSample, ESPMode::ThreadSafe>;
using FElectraAudioSampleRef = TSharedRef<FElectraAudioSample, ESPMode::ThreadSafe>;

class FElectraAudioSamplePool : public TMediaObjectPool<FElectraAudioSample>
{
public:
	~FElectraAudioSamplePool() = default;
};

#undef UE_API
