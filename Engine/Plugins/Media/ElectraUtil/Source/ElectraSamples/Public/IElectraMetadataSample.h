// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"
#include "Modules/ModuleInterface.h"
#include "Misc/Timespan.h"
#include "IMediaBinarySample.h"


/**
 * Extension of the media binary sample to carry Electra
 * specific events.
 */
class IElectraBinarySample : public IMediaBinarySample
{
public:
	static FGuid GetSampleTypeGUID()
	{
		static FGuid SampleTypeGUID(0xdfab9672, 0x754c459a, 0xa2831cb6, 0x332bab1d);
		return SampleTypeGUID;
	}

	virtual ~IElectraBinarySample() = default;
	virtual const void* GetData() = 0;
	virtual FTimespan GetDuration() const = 0;
	virtual uint32 GetSize() const = 0;
	virtual FMediaTimeStamp GetTime() const = 0;
	virtual FGuid GetGUID() const = 0;

	enum class EOrigin
	{
		EventStream,
		InbandEventStream,
		TimedMetadata
	};

	enum class EDispatchedMode
	{
		OnReceive,
		OnStart
	};

	// Returns where the event originated from.
	virtual EOrigin GetOrigin() const = 0;

	// Returns the type with which this event was dispatched.
	virtual EDispatchedMode GetDispatchedMode() const = 0;

	// Returns the event's scheme id URI
	virtual const FString& GetSchemeIdUri() const = 0;

	// Returns the event's value.
	virtual const FString& GetValue() const = 0;

	// Returns the event's ID.
	virtual const FString& GetID() const = 0;

	// Returns the base time (the "zero point") of the track, if available, to calculate the relative time
	// of this metadata from GetTime() if this is based on some non-zero anchor.
	virtual TOptional<FMediaTimeStamp> GetTrackBaseTime() const = 0;
};
