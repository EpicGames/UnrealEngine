// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"
#include "Modules/ModuleInterface.h"
#include "Misc/Timespan.h"
#include "IMediaOverlaySample.h"

/**
 * Extension of the media overlay sample to carry Electra
 * specific values.
 */
class FElectraSubtitleSample : public IMediaOverlaySample
{
public:
	virtual ~FElectraSubtitleSample() = default;

	FGuid GetGUID() const override						
	{ static FGuid SampleTypeGUID(0xC94C9B6F, 0x07DF4B88, 0xACE0B08A, 0x9EF6AA39); return SampleTypeGUID; }
	FMediaTimeStamp GetTime() const override			
	{ return Timestamp; }
	FTimespan GetDuration() const override				
	{ return Duration; }
	TOptional<FVector2D> GetPosition() const override
	{ return TOptional<FVector2D>(); }
	EMediaOverlaySampleType GetType() const override	
	{ return EMediaOverlaySampleType::Subtitle; }
	FText GetText() const override
	{ return FText::FromString(Text); }

	FString Text;
	FMediaTimeStamp Timestamp;
	FTimespan Duration;
};

