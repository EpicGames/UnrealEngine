// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IMediaTextureSample.h"
#include "IMediaTextureSampleConverter.h"
#include "Misc/Timespan.h"
#include "Misc/Timecode.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/RefCounting.h"
#include "MediaObjectPool.h"
#include "ParameterDictionary.h"
#include "IElectraTextureSampleEncoding.h"

#define UE_API ELECTRASAMPLES_API

class IVideoDecoderHDRInformation;
class IVideoDecoderColorimetry;
class IVideoDecoderTimecode;

class IElectraTextureSampleBase
	: public IMediaTextureSample
	, public IMediaPoolable
{
public:
	UE_API virtual ~IElectraTextureSampleBase();

	UE_API void InitializeCommon(const Electra::FParamDict& InParams);
	UE_API virtual bool FinishInitialization();

	DECLARE_DELEGATE_OneParam(FReleaseDelegate, IElectraTextureSampleBase*);
	UE_API FReleaseDelegate& GetReleaseDelegate();


	UE_API bool IsCacheable() const override;

#if !UE_SERVER
	UE_API void InitializePoolable() override;
	UE_API void ShutdownPoolable() override;
	UE_API bool IsReadyForReuse() override;
#endif

	UE_API FIntPoint GetDim() const override;
	UE_API FIntPoint GetOutputDim() const override;

	UE_API FMediaTimeStamp GetTime() const override;
	UE_API FTimespan GetDuration() const override;

	UE_API TOptional<FTimecode> GetTimecode() const override;
	UE_API TOptional<FFrameRate> GetFramerate() const override;

	UE_API double GetAspectRatio() const override;
	UE_API EMediaOrientation GetOrientation() const override;

	UE_API bool IsOutputSrgb() const override;
	UE_API const FMatrix& GetYUVToRGBMatrix() const override;
	UE_API bool GetFullRange() const override;

	UE_API FMatrix44f GetSampleToRGBMatrix() const override;
	UE_API const UE::Color::FColorSpace& GetSourceColorSpace() const override;
	UE_API UE::Color::EEncoding GetEncodingType() const override;
	UE_API float GetHDRNitsNormalizationFactor() const override;
	UE_API bool GetDisplayMasteringLuminance(float& OutMin, float& OutMax) const override;
	UE_API TOptional<UE::Color::FColorSpace> GetDisplayMasteringColorSpace() const override;
	UE_API bool GetMaxLuminanceLevels(uint16& OutCLL, uint16& OutFALL) const override;
	UE_API MediaShaders::EToneMapMethod GetToneMapMethod() const override;

	UE_API void SetDim(const FIntPoint& InDim);
	UE_API virtual float GetSampleDataScale(bool b10Bit) const;
	UE_API void SetTime(const FMediaTimeStamp& InTime);
	UE_API void SetDuration(const FTimespan& InDuration);
	UE_API TSharedPtr<const IVideoDecoderColorimetry, ESPMode::ThreadSafe> GetColorimetry() const;
	UE_API EPixelFormat GetPixelFormat() const;
	UE_API EElectraTextureSamplePixelEncoding GetPixelFormatEncoding() const;

protected:
	FMediaTimeStamp PresentationTimeStamp;
	FTimespan Duration;
	FIntPoint ImageDim { 0, 0 };
	FIntPoint ImageOutputDim { 0, 0 };
	int32 CropLeft = 0;
	int32 CropTop = 0;
	int32 CropRight = 0;
	int32 CropBottom = 0;

	EPixelFormat PixelFormat = EPixelFormat::PF_Unknown;
	EElectraTextureSamplePixelEncoding PixelFormatEncoding = EElectraTextureSamplePixelEncoding::Native;
	float PixelDataScale = 1.0f;

	EMediaOrientation Orientation = EMediaOrientation::Original;
	double AspectRatio = 1.0;

	/** HDR information */
	TSharedPtr<const IVideoDecoderHDRInformation, ESPMode::ThreadSafe> HDRInfo;
	TSharedPtr<const IVideoDecoderColorimetry, ESPMode::ThreadSafe> Colorimetry;
	/** Optional timecode */
	TOptional<FTimecode> Timecode;
	TOptional<FFrameRate> Framerate;

	/** YUV matrix, adjusted to compensate for decoder output specific scale */
	FMatrix44f SampleToRgbMtx;

	/** YUV to RGB matrix without any adjustments for decoder output specifics */
	const FMatrix* YuvToRgbMtx = nullptr;

	/** Precomputed colorimetric data */
	UE::Color::EEncoding ColorEncoding = UE::Color::EEncoding::None;
	UE::Color::FColorSpace SourceColorSpace;
	TOptional<UE::Color::FColorSpace> DisplayMasteringColorSpace;
	float DisplayMasteringLuminanceMin = -1.0f;
	float DisplayMasteringLuminanceMax = -1.0f;
	uint16 MaxCLL = 0;
	uint16 MaxFALL = 0;
	bool bIsFullRange = false;

	/** Optional delegate to call during ShutdownPoolable(). */
	FReleaseDelegate ReleaseDelegate;
	bool bWasShutDown = false;
};

#undef UE_API
