// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

//! Interpretation of delivered pixel data
enum class EElectraTextureSamplePixelEncoding
{
	Native = 0,		//!< Pixel formats native representation
	RGB,			//!< Interpret as RGB
	RGBA,			//!< Interpret as RGBA
	YCbCr,			//!< Interpret as YCbCR
	YCbCr_Alpha,	//!< Interpret as YCbCR with alpha
	YCoCg,			//!< Interpret as scaled YCoCg
	YCoCg_Alpha,	//!< Interpret as scaled YCoCg with trailing BC4 alpha data
	CbY0CrY1,		//!< Interpret as CbY0CrY1
	Y0CbY1Cr,		//!< Interpret as Y0CbY1Cr
	ARGB_BigEndian,	//!< Interpret as ARGB, big endian
};
