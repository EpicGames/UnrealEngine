// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IElectraTextureSample.h"
#include "RHI.h"
#include "RHIResources.h"
#include "ShaderCore.h"



class FElectraTextureSample
	: public IElectraTextureSampleBase
{
public:
	FElectraTextureSample() = default;
	ELECTRASAMPLES_API ~FElectraTextureSample();

	ELECTRASAMPLES_API bool FinishInitialization() override;

	ELECTRASAMPLES_API const void* GetBuffer() override;
	ELECTRASAMPLES_API uint32 GetStride() const override;
	ELECTRASAMPLES_API EMediaTextureSampleFormat GetFormat() const override;
	ELECTRASAMPLES_API FRHITexture* GetTexture() const override;
    ELECTRASAMPLES_API IMediaTextureSampleConverter* GetMediaTextureSampleConverter() override;

#if !UE_SERVER
	ELECTRASAMPLES_API void ShutdownPoolable() override;
#endif

	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> Buffer;
	EMediaTextureSampleFormat SampleFormat = EMediaTextureSampleFormat::Undefined;
	uint32 Stride = 0;
	bool bCanUseSRGB = false;
};

using FElectraTextureSamplePtr = TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe>;
using FElectraTextureSampleRef = TSharedRef<FElectraTextureSample, ESPMode::ThreadSafe>;

class FElectraTextureSamplePool : public TMediaObjectPool<FElectraTextureSample, FElectraTextureSamplePool>
{
	using TextureSample = FElectraTextureSample;
public:
	FElectraTextureSamplePool()
		: TMediaObjectPool<TextureSample, FElectraTextureSamplePool>(this)
	{ }
	~FElectraTextureSamplePool() = default;
	TextureSample *Alloc() const
	{ return new TextureSample(); }
};
