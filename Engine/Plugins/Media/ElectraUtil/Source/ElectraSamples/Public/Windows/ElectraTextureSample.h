// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IElectraTextureSample.h"
#include "RHI.h"
#include "RHIResources.h"
#include "ShaderCore.h"

#if !PLATFORM_WINDOWS
#error "Should only be used on Windows"
#endif

#include "Windows/AllowWindowsPlatformTypes.h"

#pragma warning(push)
#pragma warning(disable : 4005) // macro redefinition
#include <d3d11.h>
#pragma warning(pop)
#include <d3d12.h>

THIRD_PARTY_INCLUDES_START
#include "mfobjects.h"
#include "mfapi.h"
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformTypes.h"



class FElectraMediaDecoderOutputBufferPool_DX12;

class FElectraTextureSample final
	: public IElectraTextureSampleBase
	, public IMediaTextureSampleConverter
{
public:

	/** Default constructor. */
	FElectraTextureSample() = default;

	ELECTRASAMPLES_API ~FElectraTextureSample();

	ELECTRASAMPLES_API bool FinishInitialization() override;

	//
	// General Interface
	//

	ELECTRASAMPLES_API const void* GetBuffer() override;
	ELECTRASAMPLES_API uint32 GetStride() const override;
	ELECTRASAMPLES_API EMediaTextureSampleFormat GetFormat() const override;

#if WITH_ENGINE
	ELECTRASAMPLES_API FRHITexture* GetTexture() const override;
#endif //WITH_ENGINE

	ELECTRASAMPLES_API IMediaTextureSampleConverter* GetMediaTextureSampleConverter() override;

#if !UE_SERVER
	ELECTRASAMPLES_API bool IsReadyForReuse() override;
	ELECTRASAMPLES_API void ShutdownPoolable() override;
#endif

//private:
	ELECTRASAMPLES_API float GetSampleDataScale(bool b10Bit) const override;

	ELECTRASAMPLES_API bool Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints) override;
	virtual uint32 GetConverterInfoFlags() const
	{
		return ConverterInfoFlags_PreprocessOnly;
	}

	ELECTRASAMPLES_API TRefCountPtr<IUnknown> GetSync(uint64& OutSyncValue, TSharedPtr<std::atomic<bool>>& OutFenceSignalled);
	ELECTRASAMPLES_API TRefCountPtr<IUnknown> GetTextureD3D() const;


	/** The sample format. */
	EMediaTextureSampleFormat SampleFormat = EMediaTextureSampleFormat::Undefined;

	/** Destination Texture resource (from Rendering device) */
	mutable FTextureRHIRef Texture;

	/** True if texture format could support sRGB conversion in HW */
	bool bCanUseSRGB = false;

	enum class ESourceType
	{
		Unknown,
		Buffer,
		SharedTextureDX11,
		ResourceDX12
	};

	ESourceType SourceType = ESourceType::Unknown;

	// DX11 related members (output texture with device that created it)
	TRefCountPtr<ID3D11Texture2D> TextureDX11;
	TRefCountPtr<ID3D11Device> D3D11Device;

	// DX12 related members
	mutable TRefCountPtr<ID3D12Resource> TextureDX12;
	FIntPoint TextureDX12Dim {0, 0};
	TRefCountPtr<ID3D12Fence> D3DFence;
	uint64 FenceValue = 0;
	TSharedPtr<std::atomic<bool>> FenceSignalled;
	TRefCountPtr<ID3D12Resource> DecoderOutputResource;
	TRefCountPtr<ID3D12CommandAllocator> D3DCmdAllocator;
	TRefCountPtr<ID3D12GraphicsCommandList> D3DCmdList;
	TSharedPtr<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe> D3D12ResourcePool;

	// CPU-side buffer
	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> Buffer;
	uint32 Stride = 0;

	// Dimension of any internally allocated buffer - stored explicitly to cover various special cases
	FIntPoint SampleDim {0, 0};

	ELECTRASAMPLES_API void ClearDX11Vars();
	ELECTRASAMPLES_API void ClearDX12Vars();
	ELECTRASAMPLES_API void ClearBufferVars();
};

using FElectraTextureSamplePtr = TSharedPtr<FElectraTextureSample, ESPMode::ThreadSafe>;
using FElectraTextureSampleRef = TSharedRef<FElectraTextureSample, ESPMode::ThreadSafe>;

class FElectraTextureSamplePool : public TMediaObjectPool<FElectraTextureSample, FElectraTextureSamplePool>
{
	using TextureSample = FElectraTextureSample;
public:
	FElectraTextureSamplePool()
		: TMediaObjectPool<TextureSample, FElectraTextureSamplePool>(this)
	{ }
	~FElectraTextureSamplePool() = default;
	TextureSample *Alloc() const
	{ return new TextureSample(); }
};
