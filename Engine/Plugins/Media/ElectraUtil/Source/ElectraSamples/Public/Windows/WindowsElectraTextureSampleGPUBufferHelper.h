// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Queue.h"

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include "mfobjects.h"
#include "d3d12.h"
#include "d3dx12.h"
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

#define	ELECTRA_MEDIAGPUBUFFER_DX12 1

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

#define UE_API ELECTRASAMPLES_API


class FElectraMediaDecoderOutputBufferPool_DX12;

// Simple DX12 heap and fence manager for use with decoder output buffers (both upload heaps or default GPU heaps)
class FElectraMediaDecoderOutputBufferPoolBlock_DX12
{
	struct FResourceInfo;

public:
	// Create instance for use with buffers (usually for uploading data)
	UE_API FElectraMediaDecoderOutputBufferPoolBlock_DX12(uint32 InBlockIdx, TRefCountPtr<ID3D12Device> InD3D12Device, uint32 InMaxNumBuffers, uint32 Width, uint32 Height, uint32 BytesPerPixel, D3D12_HEAP_TYPE InD3D12HeapType = D3D12_HEAP_TYPE_UPLOAD);

	// Create instance for use with textures (usually to receive any upload buffers or internal decoder texture data)
	UE_API FElectraMediaDecoderOutputBufferPoolBlock_DX12(uint32 InBlockIdx, TRefCountPtr<ID3D12Device> InD3D12Device, uint32 InMaxNumBuffers, uint32 Width, uint32 Height, DXGI_FORMAT PixFmt, D3D12_HEAP_TYPE InD3D12HeapType = D3D12_HEAP_TYPE_DEFAULT);

	UE_API ~FElectraMediaDecoderOutputBufferPoolBlock_DX12();

	UE_API uint32 GetMaxNumBuffers() const;

	// Check if the current setup is compatible with the new parameters
	UE_API bool IsCompatibleAsBuffer(uint32 Width, uint32 Height, uint32 BytesPerPixel) const;

	// Check if the current setup is compatible with the new parameters
	UE_API bool IsCompatibleAsTexture(uint32 Width, uint32 Height, DXGI_FORMAT PixFmt) const;

	// Check if a buffer is available
	UE_API bool BufferAvailable() const;

	// Allocate a buffer resource and return suitable sync fence data
	UE_API TRefCountPtr<ID3D12Resource> AllocateOutputDataAsBuffer(HRESULT& OutResult, FString& OutMessage, TWeakPtr<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe> InPoolWeakRef, uint32& OutPitch);

	// Allocate a texture resource and return suitable sync fence data
	UE_API TRefCountPtr<ID3D12Resource> AllocateOutputDataAsTexture(HRESULT& OutResult, FString& OutMessage, TWeakPtr<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe> InPoolWeakRef, uint32 InWidth, uint32 InHeight, DXGI_FORMAT InPixFmt);

private:
	friend class FElectraMediaDecoderOutputBufferPool_DX12;
	void InitCommon();

	TRefCountPtr<ID3D12Resource> AllocateBuffer(HRESULT& OutResult, FString& OutMessage, TWeakPtr<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe> InPoolWeakRef, uint32& OutBufferPitch, TFunction<void(D3D12_RESOURCE_DESC& Desc)> && InitializeDesc);

	void FreeBuffer(int32 InBufferIdx);

	static void ResourceDestructionCallback(void* Context);

	TRefCountPtr<ID3D12Device> D3D12Device;
	D3D12_HEAP_TYPE D3D12HeapType;
	TRefCountPtr<ID3D12Heap> D3D12OutputHeap;
	uint64 BufferSize;
	uint32 BufferPitch;
	uint32 MaxNumBuffers;
	uint32 FreeMask;
	uint32 BlockIdx;
};


class FElectraMediaDecoderOutputBufferPool_DX12 : public TSharedFromThis<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe>
{
public:
	// Create instance for use with buffers (usually for uploading data)
	UE_API FElectraMediaDecoderOutputBufferPool_DX12(TRefCountPtr<ID3D12Device> InD3D12Device, uint32 InMaxNumBuffers, uint32 InWidth, uint32 InHeight, uint32 InBytesPerPixel, D3D12_HEAP_TYPE InD3D12HeapType = D3D12_HEAP_TYPE_UPLOAD);

	// Create instance for use with textures (usually to receive any upload buffers or internal decoder texture data)
	UE_API FElectraMediaDecoderOutputBufferPool_DX12(TRefCountPtr<ID3D12Device> InD3D12Device, uint32 InMaxNumBuffers, uint32 InWidth, uint32 InHeight, DXGI_FORMAT InPixFmt, D3D12_HEAP_TYPE InD3D12HeapType = D3D12_HEAP_TYPE_DEFAULT);

	UE_API ~FElectraMediaDecoderOutputBufferPool_DX12();

	// Check if the current setup is compatible with the new parameters
	UE_API bool IsCompatibleAsBuffer(uint32 InMaxNumBuffers, uint32 InWidth, uint32 InHeight, uint32 InBytesPerPixel) const;

	UE_API bool IsCompatibleAsTexture(uint32 InMaxNumBuffers, uint32 InWidth, uint32 InHeight, DXGI_FORMAT InPixFmt) const;

	// Check if a buffer is available
	UE_API bool BufferAvailable() const;

	class FOutputData
	{
	public:
		~FOutputData()
		{
			check(ReadyToDestroy());
		}

		TRefCountPtr<ID3D12Resource> Resource;
		TRefCountPtr<ID3D12Fence> Fence;
		uint64 FenceValue;

		bool ReadyToDestroy() const
		{
			bool bOk = true;
#if !UE_BUILD_SHIPPING
			if (Resource.IsValid())
			{
				bOk = Resource->AddRef() > 1 || Fence->GetCompletedValue() >= FenceValue;
				Resource->Release();
			}
#endif
			return bOk;
		}
	};

	UE_API bool AllocateOutputDataAsBuffer(HRESULT& OutResult, FString& OutMessage, FOutputData& OutData, uint32& OutPitch);

	UE_API bool AllocateOutputDataAsTexture(HRESULT& OutResult, FString& OutMessage, FOutputData& OutData, uint32 InWidth, uint32 InHeight, DXGI_FORMAT InPixFmt);

	UE_API TRefCountPtr<ID3D12Fence> GetUpdatedBufferFence(uint64& FenceValue);

	// Helper function to copy simple, linear texture data between differently picthed buffers
	static UE_API void CopyWithPitchAdjust(uint8* Dst, uint32 DstPitch, const uint8* Src, uint32 SrcPitch, uint32 NumRows);

private:
	friend class FElectraMediaDecoderOutputBufferPoolBlock_DX12;

	enum
	{
		kMaxBufsPerBlock = 4
	};

	void InitCommon();

	struct FPendingFreeEntry
	{
		uint32 BlockIndex;
		uint32 BufferIndex;
	};
	TQueue<FPendingFreeEntry> PendingFreeQueue;
	void AddPendingFree(uint32 InBlockIndex, uint32 InBufferIndex);
	void HandlePendingFrees();

	TArray<FElectraMediaDecoderOutputBufferPoolBlock_DX12*> Blocks;

	TRefCountPtr<ID3D12Device> D3D12Device;
	TRefCountPtr<ID3D12Fence> D3D12BufferFence;
	uint64 LastFenceValue;
	D3D12_HEAP_TYPE D3D12HeapType;
	uint32 Width;
	uint32 Height;
	DXGI_FORMAT PixFmt;
	uint32 BytesPerPixel;
	uint32 MaxNumBuffers;
	uint32 MaxBufsPerBlock;
};

#undef UE_API


#include "CommonElectraTextureSampleGPUBufferHelper.h"
